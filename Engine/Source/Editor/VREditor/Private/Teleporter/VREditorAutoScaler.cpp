// Copyright Epic Games, Inc. All Rights Reserved.

#include "Teleporter/VREditorAutoScaler.h"
#include "VREditorMode.h"
#include "ViewportInteractionTypes.h"
#include "ViewportWorldInteraction.h"
#include "ViewportInteractor.h"
#include "Sound/SoundCue.h"
#include "VREditorAssetContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorAutoScaler)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace VREd
{
	static FAutoConsoleVariable AllowResetScale(TEXT("VREd.AllowResetScale"), 1, TEXT("Allowed to reset world to meters to default world to meters"));
}

UVREditorAutoScaler::UVREditorAutoScaler() :
	Super(),
	VRMode(nullptr)
{
}

void UVREditorAutoScaler::Init( class UVREditorMode* InVRMode )
{
	VRMode = InVRMode;
	VRMode->GetWorldInteraction().OnViewportInteractionInputAction().AddUObject( this, &UVREditorAutoScaler::OnInteractorAction );
}


void UVREditorAutoScaler::Shutdown()
{
	VRMode->GetWorldInteraction().OnViewportInteractionInputAction().RemoveAll( this );
	VRMode = nullptr;
}

void UVREditorAutoScaler::Scale( const float NewWorldToMetersScale )
{
	if (VREd::AllowResetScale->GetInt() != 0)
	{
		UViewportWorldInteraction* WorldInteraction = &VRMode->GetWorldInteraction();

		// Set the new world to meters scale.
		WorldInteraction->SetWorldToMetersScale( NewWorldToMetersScale, true );
		WorldInteraction->SkipInteractiveWorldMovementThisFrame();
	
		VRMode->PlaySound(VRMode->GetAssetContainer().AutoScaleSound, VRMode->GetHeadTransform().GetLocation());
	}
}


void UVREditorAutoScaler::OnInteractorAction( class FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor, const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled )
{
	if (VREd::AllowResetScale->GetInt() != 0)
	{
		if( Action.ActionType == VRActionTypes::ConfirmRadialSelection )
		{
			if( Action.Event == IE_Pressed )
			{
				const EViewportInteractionDraggingMode DraggingMode = Interactor->GetDraggingMode();
				UViewportInteractor* OtherInteractor = Interactor->GetOtherInteractor();

				// Start dragging at laser impact when already dragging actors freely
				if( DraggingMode == EViewportInteractionDraggingMode::World ||
					( DraggingMode == EViewportInteractionDraggingMode::AssistingDrag && OtherInteractor != nullptr && OtherInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::World ) )
				{
					const float DefaultWorldToMetersScale = VRMode->GetSavedEditorState().WorldToMetersScale;
					Scale( DefaultWorldToMetersScale );

					bWasHandled = true;
					bOutIsInputCaptured = true;
				}
			}
			else if( Action.Event == IE_Released )
			{
				bOutIsInputCaptured = false;
			}
		}
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
