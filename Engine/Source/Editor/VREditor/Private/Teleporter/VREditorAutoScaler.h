// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VREditorAutoScaler.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UVREditorMode;


/**
 * Automatically scales the user when dragging the world and pressing the touchpad
 */
UCLASS()
class UVREditorAutoScaler: public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UVREditorAutoScaler();

	/** Initializes the automatic scaler */
	void Init( class UVREditorMode* InVRMode );

	/** Shuts down the automatic scaler */
	void Shutdown();

private:

	/** Actual scale to a certain world to meters */
	void Scale( const float NewScale );

	/** Called when the user presses a button on their motion controller device */
	void OnInteractorAction( class FEditorViewportClient& ViewportClient, class UViewportInteractor* Interactor,
		const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled );

	/** Owning mode */
	UPROPERTY()
	TObjectPtr<UVREditorMode> VRMode;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
