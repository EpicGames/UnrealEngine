// Copyright Epic Games, Inc. All Rights Reserved.

#include "Teleporter/VREditorTeleporter.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "HAL/IConsoleManager.h"
#include "HeadMountedDisplayTypes.h"
#include "Kismet/GameplayStatics.h"
#include "Sound/SoundCue.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ViewportInteractionTypes.h"
#include "ViewportWorldInteraction.h"
#include "Teleporter/VREditorTeleporter.h"
#include "VREditorMode.h"
#include "VREditorAssetContainer.h"
#include "VREditorInteractor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorTeleporter)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace VREd
{
	static FAutoConsoleVariable TeleportLerpTime(TEXT("VREd.TeleportLerpTime"), 0.1f, TEXT("The lerp time to teleport"));
	static FAutoConsoleVariable TeleportOffset(TEXT("VREd.TeleportOffset"), 100.0f, TEXT("The offset from the hitresult towards the controller"));
	static FAutoConsoleVariable TeleportLaserPointerLength(TEXT("VREd.TeleportLaserPointerLength"), 500000.0f, TEXT("Distance of the LaserPointer for teleporting"));
	static FAutoConsoleVariable TeleportDistance(TEXT("VREd.TeleportDistance"), 500.0f, TEXT("Default distance for teleporting when not hitting anything"));
	static FAutoConsoleVariable TeleportScaleSensitivity(TEXT("VREd.TeleportScaleSensitivity"), 0.05f, TEXT("Teleport world to meters scale touchpad sensitivity"));
	static FAutoConsoleVariable TeleportOffsetMultiplier(TEXT("VREd.TeleportOffsetMultiplier"), 0.3f, TEXT("Teleport offset multiplier"));
	static FAutoConsoleVariable TeleportEnableChangeScale(TEXT("VREd.TeleportEnableChangeScale"), 0, TEXT("Ability to change the world to meters scale while teleporting"));
	static FAutoConsoleVariable TeleportFadeInAnimateSpeed(TEXT("VREd.TeleportAnimateSpeed"), 3.0f, TEXT("How fast the teleporter should fade in"));
	static FAutoConsoleVariable TeleportDragSpeed(TEXT("VREd.TeleportDragSpeed"), 0.3f, TEXT("How fast the teleporter should drag behind the laser aiming location"));
	static FAutoConsoleVariable TeleportAllowScaleBackToDefault(TEXT("VREd.TeleportAllowScaleBackToDefault"), 1, TEXT("Scale back to default world to meters scale"));
	static FAutoConsoleVariable TeleportAllowPushPull(TEXT("VREd.TeleportAllowPushPull"), 1, TEXT("Allow being able to push and pull the teleporter along the laser."));
	static FAutoConsoleVariable TeleportSlideBuffer(TEXT("VREd.TeleportSlideBuffer"), 0.01f, TEXT("The minimum slide on trackpad to push/pull or change scale."));
}

AVREditorTeleporter::AVREditorTeleporter():
	Super(),
	VRMode(nullptr),
	TeleportingState(EState::None),
	TeleportLerpAlpha(0),
	TeleportStartLocation(FVector::ZeroVector),
	TeleportGoalLocation(FVector::ZeroVector),
	TeleportDirectionMeshComponent(nullptr),
	HMDMeshComponent(nullptr),
	LeftMotionControllerMeshComponent(nullptr),
	RightMotionControllerMeshComponent(nullptr),
	TeleportMID(nullptr),
	InteractorTryingTeleport(nullptr),
	OffsetDistance(FVector::ZeroVector),
	TeleportGoalScale(0),
	DragRayLength(0),
	DragRayLengthVelocity(0),
	bPushedFromEndOfLaser(false),
	bInitialTeleportAim(true),
	FadeAlpha(1.0f),
	bShouldBeVisible(),
	TeleportTickDelay(0)
{
}

void AVREditorTeleporter::Init_Implementation(class UVREditorMode* InMode)
{
	VRMode = InMode;

	// Register for event for ticking and input
	VRMode->OnTickHandle().AddUObject(this, &AVREditorTeleporter::Tick);
	VRMode->GetWorldInteraction().OnPreviewInputAction().AddUObject(this, &AVREditorTeleporter::OnPreviewInputAction);

	SetActorEnableCollision(false);

	const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();

	{
		UMaterialInterface* RoomSpaceMaterial = AssetContainer.TeleportMaterial;
		check(RoomSpaceMaterial != nullptr);
		TeleportMID = UMaterialInstanceDynamic::Create(RoomSpaceMaterial, this);
		check(TeleportMID != nullptr);
	}

	{
		TeleportDirectionMeshComponent = VRMode->CreateMesh(this, AssetContainer.TeleportRootMesh, GetRootComponent());
		TeleportDirectionMeshComponent->SetWorldScale3D(FVector(1, 1, 1));
		TeleportDirectionMeshComponent->SetMaterial(0, TeleportMID);
		TeleportDirectionMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		TeleportDirectionMeshComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);
		TeleportDirectionMeshComponent->SetCollisionResponseToChannel(COLLISION_GIZMO, ECollisionResponse::ECR_Ignore);
		TeleportDirectionMeshComponent->SetCastShadow(false);

		HMDMeshComponent = VRMode->CreateMesh(this, AssetContainer.GenericHMDMesh, GetRootComponent());
		HMDMeshComponent->SetMaterial(0, TeleportMID);
		HMDMeshComponent->SetCastShadow(false);

		LeftMotionControllerMeshComponent = VRMode->CreateMotionControllerMesh(this, GetRootComponent());
		check(LeftMotionControllerMeshComponent != nullptr);
		LeftMotionControllerMeshComponent->SetCastShadow(false);

		RightMotionControllerMeshComponent = VRMode->CreateMotionControllerMesh(this, GetRootComponent());
		check(RightMotionControllerMeshComponent != nullptr);
		RightMotionControllerMeshComponent->SetCastShadow(false);

		LeftMotionControllerMeshComponent->SetMaterial(0, TeleportMID);
		RightMotionControllerMeshComponent->SetMaterial(0, TeleportMID);
	}

	// Set the initial teleport scale to the current world to meters
	TeleportGoalScale = VRMode->GetSavedEditorState().WorldToMetersScale;

	Show(false);
}

void AVREditorTeleporter::Shutdown_Implementation()
{
	VRMode->OnTickHandle().RemoveAll(this);
	VRMode->GetWorldInteraction().OnViewportInteractionInputAction().RemoveAll(this);
	VRMode = nullptr;
}

bool AVREditorTeleporter::IsAiming() const
{
	return TeleportingState == EState::Aiming;
}

bool AVREditorTeleporter::IsTeleporting() const
{
	return TeleportingState == EState::Teleporting;
}

class UVREditorMode* AVREditorTeleporter::GetVRMode() const
{
	return VRMode;
}

void AVREditorTeleporter::Tick(const float DeltaTime)
{
	UpdateFadingState(DeltaTime);
	if (TeleportingState == EState::Aiming && InteractorTryingTeleport != nullptr)
	{
		{
			UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>(InteractorTryingTeleport);
			if ((VREditorInteractor != nullptr && (VREditorInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::World ||
				(VREditorInteractor->GetOtherInteractor() != nullptr && VREditorInteractor->GetOtherInteractor()->GetInteractorData().DraggingMode == EViewportInteractionDraggingMode::World &&
					VREditorInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::AssistingDrag)) &&
				(!FMath::IsNearlyZero(VREditorInteractor->GetSelectAndMoveTriggerValue(), KINDA_SMALL_NUMBER) || (VREditorInteractor->GetControllerType() == EControllerType::Navigation))))
			{
				VREditorInteractor->SetForceShowLaser(true);
			}
		}

		UpdateTeleportAim(DeltaTime);
	}
	else if (TeleportingState == EState::Teleporting)
	{
		static const uint32 StaticTeleportDelay = 2;
		if (TeleportTickDelay > StaticTeleportDelay)
		{
			Teleport(DeltaTime);
		}
		TeleportTickDelay++;
	}
}

void AVREditorTeleporter::StartTeleport_Implementation()
{

	if (VREd::TeleportEnableChangeScale->GetInt() != 0 || VREd::TeleportAllowScaleBackToDefault->GetInt() != 0)
	{
		// Set the world to meters scale @todo vreditor: This causes some strange movement
		VRMode->GetWorldInteraction().SetWorldToMetersScale(TeleportGoalScale * 100, true);
	}

	TeleportingState = EState::Teleporting;
	TeleportLerpAlpha = 0.0f;

	const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();
	VRMode->PlaySound(AssetContainer.TeleportSound, TeleportGoalLocation);
	InteractorTryingTeleport->PlayHapticEffect(1.0f);
}

void AVREditorTeleporter::StartAiming( class UViewportInteractor* Interactor )
{
	if (!IsTeleporting() && Interactor != nullptr)
	{
		TeleportGoalScale = VRMode->GetWorldScaleFactor();

		// Checking teleport
		SetVisibility(true);
		InteractorTryingTeleport = Interactor;

		TeleportingState = EState::Aiming;
		SetColor(VRMode->GetColor(UVREditorMode::EColors::WorldDraggingColor));
		VRMode->GetWorldInteraction().AllowWorldMovement(false);
	}
}

void AVREditorTeleporter::StopAiming( )
{
	if (!IsTeleporting())
	{
		bPushedFromEndOfLaser = false;
		bInitialTeleportAim = true;
		bShouldBeVisible = false;
		SetVisibility(false);

		if (InteractorTryingTeleport != nullptr)
		{
			UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>( InteractorTryingTeleport );
			VREditorInteractor->SetForceShowLaser(false);
		}
		InteractorTryingTeleport = nullptr;

		TeleportingState = EState::None;
		VRMode->GetWorldInteraction().AllowWorldMovement(true);
	}
}

void AVREditorTeleporter::DoTeleport()
{
	if (!IsTeleporting())
	{
		UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>( InteractorTryingTeleport );
		if (VREditorInteractor && VREditorInteractor == InteractorTryingTeleport && !(VREditorInteractor->IsHoveringOverGizmo() || VREditorInteractor->IsHoveringOverUI()))
		{
			// Confirm teleport
			StartTeleport( );
		}
	}
}

UViewportInteractor* AVREditorTeleporter::GetInteractorTryingTeleport() const
{
	return InteractorTryingTeleport;
}

void AVREditorTeleporter::OnPreviewInputAction(class FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor, const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled)
{
	// see if we're already teleporting with another interactor.  If so, bail
	if (InteractorTryingTeleport != nullptr && InteractorTryingTeleport != Interactor)
	{
		return;
	}

	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>(Interactor);

	if (VREditorInteractor != nullptr && VREditorInteractor->GetControllerType() == EControllerType::Laser)
	{
		const bool bDraggingWorldForTeleport =
			(Interactor->GetDraggingMode () == EViewportInteractionDraggingMode::World &&
			(Interactor->GetOtherInteractor () == nullptr || Interactor->GetOtherInteractor ()->GetDraggingMode () != EViewportInteractionDraggingMode::AssistingDrag)) ||
				(VREditorInteractor->GetControllerType() == EControllerType::Navigation && (Action.ActionType == VRActionTypes::TrackpadDown));
		if (bDraggingWorldForTeleport &&
			Action.Event == IE_Pressed &&
			Action.ActionType == ViewportWorldActionTypes::SelectAndMove)
		{
			if (InteractorTryingTeleport == nullptr && TeleportingState == EState::None)
			{
				TeleportGoalScale = VRMode->GetWorldScaleFactor ();

				// Checking teleport
				SetVisibility ( true );
				InteractorTryingTeleport = Interactor;

				TeleportingState = EState::Aiming;
				SetColor ( VRMode->GetColor ( UVREditorMode::EColors::WorldDraggingColor ) );
				VRMode->GetWorldInteraction ().AllowWorldMovement ( false );
			}

			bWasHandled = true;
		}

		if (Action.Event == IE_Released &&
			Action.ActionType == ViewportWorldActionTypes::SelectAndMove &&
			InteractorTryingTeleport != nullptr && InteractorTryingTeleport == Interactor)
		{
			if (!(VREditorInteractor->IsHoveringOverGizmo () || VREditorInteractor->IsHoveringOverUI ()))
			{
				// Confirm teleport
				StartTeleport ( );
				// Clean everything up
				bPushedFromEndOfLaser = false;
				bInitialTeleportAim = true;
				bShouldBeVisible = false;
				SetVisibility ( false );

				VREditorInteractor->SetForceShowLaser ( false );
				InteractorTryingTeleport = nullptr;

				bWasHandled = true;
			}
		}
	}
}

void AVREditorTeleporter::Teleport(const float DeltaTime)
{
	if (TeleportLerpAlpha == 0.0f)
	{
		TeleportStartLocation = VRMode->GetRoomTransform().GetLocation();
	}

	FTransform RoomTransform = VRMode->GetRoomTransform();
	TeleportLerpAlpha += DeltaTime;
	if (TeleportLerpAlpha > VREd::TeleportLerpTime->GetFloat())
	{
		// Teleporting is finished
		TeleportDone();
	}

	// Calculate the new position of the roomspace
	FVector NewLocation = FMath::Lerp(TeleportStartLocation, TeleportGoalLocation, TeleportLerpAlpha / VREd::TeleportLerpTime->GetFloat());
	RoomTransform.SetLocation(NewLocation);
	VRMode->SetRoomTransform(RoomTransform);
}

void AVREditorTeleporter::TeleportDone_Implementation()
{
	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>( InteractorTryingTeleport );

	// if called from interactor navigation mode we might still be touching the trackpad and want to teleport again.
	if (VREditorInteractor != nullptr && VREditorInteractor->IsTouchingTrackpad())
	{
		TeleportingState = EState::Aiming;
	}
	else
	{
		TeleportingState = EState::None;
	}

	TeleportLerpAlpha = VREd::TeleportLerpTime->GetFloat();
	VRMode->GetWorldInteraction().AllowWorldMovement( true );
	TeleportTickDelay = 0;
}

void AVREditorTeleporter::UpdateTeleportAim(const float DeltaTime)
{
	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>(InteractorTryingTeleport);
	if (VREditorInteractor)
	{
		Show(!(VREditorInteractor->IsHoveringOverGizmo() || VREditorInteractor->IsHoveringOverUI()));

		FVector LaserPointerStart, LaserPointerEnd, EndLocation;
		if (VREditorInteractor->GetLaserPointer(LaserPointerStart, LaserPointerEnd) && TeleportingState == EState::Aiming)
		{
			bool bAllowPushPull = VREd::TeleportAllowPushPull->GetInt() == 0 ? false : true;

			if (VREd::TeleportEnableChangeScale->GetInt() != 0)
			{
				const float SlideDelta = GetSlideDelta(VREditorInteractor, 0);
				if (SlideDelta != 0.0f)
				{
					// Calculate the new goal scale with the trackpad delta X axis
					TeleportGoalScale += SlideDelta * (TeleportGoalScale * VREd::TeleportScaleSensitivity->GetFloat());

					const float MinScale = VRMode->GetWorldInteraction().GetMinScale() * 0.01f;
					const float MaxScale = VRMode->GetWorldInteraction().GetMaxScale() * 0.01f;

					// The scale must be in the limits of the worldinteraction minimum and maximum scale
					TeleportGoalScale = FMath::Clamp(TeleportGoalScale, MinScale, MaxScale);
					bAllowPushPull = false;
				}
			}
			else if (VREd::TeleportAllowScaleBackToDefault->GetInt() != 0)
			{
				TeleportGoalScale = VRMode->GetSavedEditorState().WorldToMetersScale / 100.f;
			}
			else
			{
				TeleportGoalScale = VRMode->GetWorldScaleFactor();
			}

			// If the laser is hitting something the teleport will go their with an appropriate offset.
			FHitResult HitResult = VREditorInteractor->GetHitResultFromLaserPointer(nullptr, EHitResultGizmoFilterMode::NoGizmos, nullptr, false, VREd::TeleportLaserPointerLength->GetFloat());
			if (HitResult.bBlockingHit && !bPushedFromEndOfLaser)
			{
				// Calculate an offset with the impact normal, so the teleporter will show up on-top, underneath or next to where laser is aiming at.
				OffsetDistance = HitResult.ImpactNormal * (50 * TeleportGoalScale);
				double OffsetFromImpactNormalZ = (HitResult.ImpactNormal.Z - 1.0f);
				OffsetFromImpactNormalZ -= (OffsetFromImpactNormalZ * VREd::TeleportOffsetMultiplier->GetFloat());
				OffsetDistance.Z = OffsetFromImpactNormalZ * (((VRMode->GetHeadTransform().GetLocation().Z - VRMode->GetRoomTransform().GetLocation().Z) / VRMode->GetWorldScaleFactor())  * TeleportGoalScale);

				// Set the final location based on the hit location and the offset
				EndLocation = HitResult.Location + OffsetDistance;

				// Update the raylength to the current length, so if we have to pull or push the teleporter next frame it will not jump
				DragRayLength = FVector::Dist(LaserPointerStart, HitResult.ImpactPoint);
			}
			// If the laser is not hitting anything or has been pushed away already, the user can push or pull the teleporter along the laser
			else
			{
				EndLocation = UpdatePushPullTeleporter(VREditorInteractor, LaserPointerStart, LaserPointerEnd, bAllowPushPull);
			}

			// The trackpad has been used while aiming for teleporting, so the teleporter won't go to the end of the laser after this
			if (VREditorInteractor->GetControllerType() != EControllerType::Navigation && !bPushedFromEndOfLaser && bAllowPushPull &&  GetSlideDelta(VREditorInteractor, 1) != 0.0f)
			{
				bPushedFromEndOfLaser = true;
			}

			// Smooth the final location
			if (!bInitialTeleportAim)
			{
				FVector TeleporterAndAimLocationOffset = EndLocation - GetActorLocation();
				if (TeleporterAndAimLocationOffset.Size() > (VRMode->GetWorldScaleFactor() * 1.0f)) //@todo vreditor: Tweak this value for less jitter
				{
					EndLocation = GetActorLocation() + (TeleporterAndAimLocationOffset * VREd::TeleportDragSpeed->GetFloat());
				}
			}

			UpdateVisuals(EndLocation);
			bInitialTeleportAim = false;
		}
	}
}

FVector AVREditorTeleporter::UpdatePushPullTeleporter(UVREditorInteractor* VREditorInteractor, const FVector& LaserPointerStart, const FVector& LaserPointerEnd, const bool bEnablePushPull /* = true */)
{
	if (bEnablePushPull && GetSlideDelta(VREditorInteractor, 1) != 0.0f)
	{
		VREditorInteractor->CalculateDragRay(DragRayLength, DragRayLengthVelocity);
	}

	FVector Direction = LaserPointerEnd - LaserPointerStart;
	Direction.Normalize();

	return (LaserPointerStart + (Direction * DragRayLength)) + OffsetDistance;
}

void AVREditorTeleporter::SetVisibility(const bool bVisible)
{
	TeleportDirectionMeshComponent->SetVisibility(bVisible);
	HMDMeshComponent->SetVisibility(bVisible);
	LeftMotionControllerMeshComponent->SetVisibility(bVisible);
	RightMotionControllerMeshComponent->SetVisibility(bVisible);
}

void AVREditorTeleporter::SetColor(const FLinearColor& Color)
{
	static FName StaticRoomspaceMeshColorName("Color");
	TeleportMID->SetVectorParameterValue(StaticRoomspaceMeshColorName, Color);
}

void AVREditorTeleporter::UpdateVisuals(const FVector& NewLocation)
{
	const float WorldScale = VRMode->GetWorldScaleFactor();
	const float CalculatedScale = CalculateAnimatedScaleFactor();
	const FVector AnimatedScale = FVector(TeleportGoalScale * CalculatedScale);

	SetActorLocation(NewLocation);
	FTransform HMDTransform = VRMode->GetHeadTransform();

	// Update the teleport direction indicator
	{
		const FRotator TeleportDirection = FRotator(0.f, HMDTransform.GetRotation().Rotator().Yaw, 0.f);
		TeleportDirectionMeshComponent->SetWorldRotation(TeleportDirection);
		TeleportDirectionMeshComponent->SetWorldScale3D(AnimatedScale);
		TeleportDirectionMeshComponent->SetRelativeLocation(FVector(0.0f, 0.0f, AnimatedScale.Z * 0.5f));
	}

	HMDTransform.SetLocation(FVector(NewLocation.X, NewLocation.Y, NewLocation.Z + ((VRMode->GetRoomSpaceHeadTransform().GetLocation().Z / WorldScale) * TeleportGoalScale)));
	HMDTransform.SetScale3D(AnimatedScale);
	HMDMeshComponent->SetWorldTransform(HMDTransform);

	//Calculate the teleported room transform
	FTransform HeadToWorld = VRMode->GetHeadTransform();
	HeadToWorld.SetRotation(FQuat::Identity);
	const FTransform RoomToWorld = VRMode->GetRoomTransform();
	FTransform RoomToHeadInWorld = RoomToWorld.GetRelativeTransform(HeadToWorld);
	RoomToHeadInWorld.SetLocation(FVector((RoomToHeadInWorld.GetLocation() / WorldScale) * TeleportGoalScale));

	HMDTransform.SetRotation(FQuat::Identity);
	FTransform TeleportRoomInWorld = HMDTransform + RoomToHeadInWorld;
	TeleportGoalLocation = TeleportRoomInWorld.GetLocation();

	// Calculate the teleported motion controllers
	for (int i = 0; i < 2; i++)
	{
		UVREditorInteractor* Interactor = i == 0 ? VRMode->GetHandInteractor(EControllerHand::Left) : VRMode->GetHandInteractor(EControllerHand::Right);
		UStaticMeshComponent* MotionControllerMeshComponent = i == 0 ? LeftMotionControllerMeshComponent : RightMotionControllerMeshComponent;

		FTransform MotionControllerToWorld = Interactor->GetTransform();
		FTransform MotionControllerToHeadInWorld = MotionControllerToWorld.GetRelativeTransform(HeadToWorld);
		MotionControllerToHeadInWorld.SetLocation(FVector((MotionControllerToHeadInWorld.GetLocation() / WorldScale) * TeleportGoalScale));
		FTransform TeleportedMotionControllerToWorld = HMDTransform + MotionControllerToHeadInWorld;
		TeleportedMotionControllerToWorld.SetRotation(MotionControllerToWorld.GetRotation());
		TeleportedMotionControllerToWorld.SetScale3D(AnimatedScale);

		MotionControllerMeshComponent->SetWorldTransform(TeleportedMotionControllerToWorld);
	}
}

void AVREditorTeleporter::Show(const bool bShow)
{
	// Only start the fading in or out if it is the first time requesting it. Only reset animation if it wasn't started yet.
	if (bShow)
	{
		if (!bShouldBeVisible.IsSet() || bShouldBeVisible.GetValue() != bShow)
		{
			bShouldBeVisible = true;
			FadeAlpha = 0.0f;
		}
	}
	else
	{
		bShouldBeVisible = false;
		SetVisibility(false);
	}
}

void AVREditorTeleporter::UpdateFadingState(const float DeltaTime)
{
	if (bShouldBeVisible.IsSet() && bShouldBeVisible.GetValue())
	{
		FadeAlpha += VREd::TeleportFadeInAnimateSpeed->GetFloat() * DeltaTime;
		FadeAlpha = FMath::Clamp(FadeAlpha, 0.0f, 1.0f);

		if (FadeAlpha > 0.0f + KINDA_SMALL_NUMBER)
		{
			// At least a little bit visible
			if (!TeleportDirectionMeshComponent->IsVisible())
			{
				SetVisibility(true);
			}
		}
	}
}

float AVREditorTeleporter::CalculateAnimatedScaleFactor() const
{
	const float AnimationOvershootAmount = 0.7f;	// @todo vreditor tweak
	const float EasedAlpha = UVREditorMode::OvershootEaseOut(FadeAlpha, AnimationOvershootAmount);

	// Animate vertically more than horizontally; just looks a little better
	const float Scale = FMath::Max(0.1f, EasedAlpha);
	return Scale * Scale;
}

float AVREditorTeleporter::GetSlideDelta_Implementation(UVREditorInteractor* Interactor, const bool Axis)
{
	static const FName SteamVR(TEXT("SteamVR"));

	FVector2D SlideDelta = FVector2D::ZeroVector;
	const bool bIsAbsolute = (VRMode->GetHMDDeviceType() == SteamVR);
	if (bIsAbsolute)
	{
		SlideDelta = FVector2D(Interactor->GetTrackpadSlideDelta(0), Interactor->GetTrackpadSlideDelta(1));
	}
	else
	{
		SlideDelta = Interactor->GetTrackpadPosition();
	}

	float Result = 0.0f;
	if (FMath::Abs(SlideDelta[Axis]) > FMath::Abs(SlideDelta[!Axis]))
	{
		Result = (float) SlideDelta[Axis];
	}

	return Result;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
