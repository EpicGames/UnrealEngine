// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorBaseUserWidget.h"
#include "UI/VREditorFloatingUI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorBaseUserWidget)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UVREditorBaseUserWidget::UVREditorBaseUserWidget( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer ),
	  Owner( nullptr )
{
}


void UVREditorBaseUserWidget::SetOwner( class AVREditorFloatingUI* NewOwner )
{
	Owner = NewOwner;
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
