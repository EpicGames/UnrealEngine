// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Blueprint/UserWidget.h"
#include "VREditorBaseUserWidget.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class AVREditorFloatingUI;

/**
 * Base class for all of the VR widgets
 */
UCLASS( Abstract )
class UVREditorBaseUserWidget : public UUserWidget
{
	GENERATED_BODY()

public:
	
	/** Default constructor that sets up CDO properties */
	UVREditorBaseUserWidget( const FObjectInitializer& ObjectInitializer );

	/** Gets the owner of this widget */
	class AVREditorFloatingUI* GetOwner()
	{
		return Owner.Get();
	}

	/** Gets the owner of this widget (const) */
	const class AVREditorFloatingUI* GetOwner() const
	{
		return Owner.Get();
	}

	/** Sets the UI system that owns this widget */
	void SetOwner( class AVREditorFloatingUI* NewOwner );


protected:

	/** The UI system that owns this widget */
	UPROPERTY()
	TWeakObjectPtr<class AVREditorFloatingUI> Owner;

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
