// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorDockableCameraWindow.h"
#include "VREditorCameraWidgetComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorDockableCameraWindow)

AVREditorDockableCameraWindow::AVREditorDockableCameraWindow(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer.SetDefaultSubobjectClass<UVREditorCameraWidgetComponent>(TEXT("WidgetComponent")))
{
}

