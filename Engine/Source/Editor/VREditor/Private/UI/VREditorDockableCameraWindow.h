// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/VREditorDockableWindow.h"
#include "VREditorDockableCameraWindow.generated.h"

/**
 * A specialized dockable window for camera viewfinders in VR that applies the correct material
 */
UCLASS()
class AVREditorDockableCameraWindow : public AVREditorDockableWindow
{
	GENERATED_BODY()
	
public:

	/** Default constructor */
  AVREditorDockableCameraWindow(const FObjectInitializer& ObjectInitializer);
};

