// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorDockableWindow.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/Material.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UI/VREditorUISystem.h"
#include "VREditorMode.h"
#include "VREditorWidgetComponent.h"
#include "ViewportWorldInteraction.h"
#include "VREditorInteractor.h"
#include "VREditorAssetContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorDockableWindow)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable DockWindowThickness( TEXT( "VREd.DockWindowTickness" ), 1.0f, TEXT( "How thick the window is" ) );
	static FAutoConsoleVariable DockUISelectionBarVerticalOffset( TEXT( "VREd.DockUISelectionBarVerticalOffset" ), 2.0f, TEXT( "Z Distance between the selectionbar and the UI" ) );
	static FAutoConsoleVariable DockUIFadeAnimationDuration( TEXT( "VREd.DockUIFadeAnimationDuration" ), 0.15f, TEXT( "How quick the fade animation should complete in" ) );
	static FAutoConsoleVariable DockUIHoverScale( TEXT( "VREd.DockUIHoverScale" ), 1.1f, TEXT( "How big the selection bar gets when you hover over it" ) );
	static FAutoConsoleVariable DockUIHoverAnimationDuration( TEXT( "VREd.DockUIHoverAnimationDuration" ), 0.15f, TEXT( "How quick the hover animation should complete in" ) );
	static FAutoConsoleVariable DockUIDragSmoothingAmount( TEXT( "VREd.DockUIDragSmoothingAmount" ), 0.85f, TEXT( "How much to smooth out motion when dragging UI (frame rate sensitive)" ) );
}


AVREditorDockableWindow::AVREditorDockableWindow(const FObjectInitializer& ObjectInitializer) :
	Super(ObjectInitializer),
	bIsLaserAimingTowardUI(false),
	AimingAtMeFadeAlpha(0.0f),
	bIsHoveringOverSelectionBar(false),
	SelectionBarHoverAlpha(0.0f),
	bIsHoveringOverCloseButton(false),
	CloseButtonHoverAlpha(0.0f),
	DockSelectDistance(0.0f)
{
}

void AVREditorDockableWindow::PostActorCreated()
{
	Super::PostActorCreated();

	const UVREditorAssetContainer& AssetContainer = UVREditorMode::LoadAssetContainer();

	// Note that this might be overridden if the CreationContext defines a mesh. But we don't have the CC yet, so we're setting the default here
	SetWindowMesh(AssetContainer.WindowMesh);

	UMaterialInterface* HoverMaterial = AssetContainer.WindowMaterial;
	UMaterialInterface* TranslucentHoverMaterial = AssetContainer.WindowMaterial;

	const FRotator RelativeRotation(30.f, 0.f, 0.f);
	{
		UStaticMesh* DockingMesh = AssetContainer.DockingButtonMesh;

		DockButtonMeshComponent = NewObject<UStaticMeshComponent>(this, TEXT( "DockMesh" ) );
		DockButtonMeshComponent->SetupAttachment(RootComponent);
		DockButtonMeshComponent->RegisterComponent();
		DockButtonMeshComponent->SetStaticMesh(DockingMesh);
		DockButtonMeshComponent->SetMobility( EComponentMobility::Movable );

		DockButtonMeshComponent->SetGenerateOverlapEvents(false);
		DockButtonMeshComponent->SetCanEverAffectNavigation( false );
		DockButtonMeshComponent->bCastDynamicShadow = false;
		DockButtonMeshComponent->bCastStaticShadow = false;
		DockButtonMeshComponent->bAffectDistanceFieldLighting = false;
		DockButtonMeshComponent->SetRelativeRotation(RelativeRotation);


		DockButtonMID = UMaterialInstanceDynamic::Create( HoverMaterial, GetTransientPackage() );
		check(DockButtonMID != nullptr );
		DockButtonMeshComponent->SetMaterial( 0, DockButtonMID);
		DockButtonMeshComponent->SetMaterial(1, DockButtonMID);

		UStaticMesh* SelectionMesh = AssetContainer.WindowSelectionBarMesh;

		SelectionBarMeshComponent = NewObject<UStaticMeshComponent>(this, TEXT("SelectionBarMesh"));
		SelectionBarMeshComponent->SetupAttachment(RootComponent);
		SelectionBarMeshComponent->SetMobility(EComponentMobility::Movable);
		SelectionBarMeshComponent->RegisterComponent();
		SelectionBarMeshComponent->SetStaticMesh(SelectionMesh);
		SelectionBarMeshComponent->SetGenerateOverlapEvents(false);
		SelectionBarMeshComponent->SetCanEverAffectNavigation(false);
		SelectionBarMeshComponent->bCastDynamicShadow = false;
		SelectionBarMeshComponent->bCastStaticShadow = false;
		SelectionBarMeshComponent->bAffectDistanceFieldLighting = false;
		SelectionBarMeshComponent->SetRelativeRotation(RelativeRotation);


		SelectionBarMID = UMaterialInstanceDynamic::Create(HoverMaterial, GetTransientPackage());
		check(SelectionBarMID != nullptr);
		SelectionBarMeshComponent->SetMaterial(0, SelectionBarMID);
		SelectionBarTranslucentMID = UMaterialInstanceDynamic::Create(TranslucentHoverMaterial, GetTransientPackage());
		check(SelectionBarTranslucentMID != nullptr);
		SelectionBarMeshComponent->SetMaterial(1, SelectionBarTranslucentMID);
	}

	{
		UStaticMesh* CloseButtonMesh = AssetContainer.WindowCloseButtonMesh;

		CloseButtonMeshComponent = NewObject<UStaticMeshComponent>(this, TEXT("CloseButtonMesh"));
		CloseButtonMeshComponent->SetStaticMesh(CloseButtonMesh);
		CloseButtonMeshComponent->SetMobility(EComponentMobility::Movable);
		CloseButtonMeshComponent->SetupAttachment(RootComponent);
		CloseButtonMeshComponent->RegisterComponent();
		CloseButtonMeshComponent->SetGenerateOverlapEvents(false);
		CloseButtonMeshComponent->SetCanEverAffectNavigation(false);
		CloseButtonMeshComponent->bCastDynamicShadow = false;
		CloseButtonMeshComponent->bCastStaticShadow = false;
		CloseButtonMeshComponent->bAffectDistanceFieldLighting = false;
		CloseButtonMeshComponent->SetRelativeRotation(RelativeRotation);

		CloseButtonMID = UMaterialInstanceDynamic::Create(HoverMaterial, GetTransientPackage());
		check(CloseButtonMID != nullptr);
		CloseButtonMeshComponent->SetMaterial(0, CloseButtonMID);
		CloseButtonTranslucentMID = UMaterialInstanceDynamic::Create(TranslucentHoverMaterial, GetTransientPackage());
		check(CloseButtonTranslucentMID != nullptr);
		CloseButtonMeshComponent->SetMaterial(1, CloseButtonTranslucentMID);
	}

	// The selection bar and close button will not be initially visible.  They'll appear when the user aims
	// their laser toward the UI
	SelectionBarMeshComponent->SetVisibility(false);
	CloseButtonMeshComponent->SetVisibility(false);

	// Create the drag operation
	DragOperationComponent = NewObject<UViewportDragOperationComponent>(this, TEXT("DragOperation"));
	DragOperationComponent->SetDragOperationClass(UDockableWindowDragOperation::StaticClass());
}



void AVREditorDockableWindow::SetupWidgetComponent()
{
	Super::SetupWidgetComponent();
	// In standard UED, dockable UIs always have a border so we need to make the background not transparent unless...
	// The CreationContext overrides the styling.
	if (CreationContext.bMaskOutWidgetBackground)
	{
		WidgetComponent->SetOpacityFromTexture(1.0f);
		WidgetComponent->SetBackgroundColor(FLinearColor::Transparent);
		WidgetComponent->SetBlendMode(EWidgetBlendMode::Transparent);

	}
	else
	{
		WidgetComponent->SetOpacityFromTexture(0.0f);
		WidgetComponent->SetBackgroundColor(FLinearColor::Black);
		WidgetComponent->SetBlendMode(EWidgetBlendMode::Opaque);

	}
	SetSelectionBarColor( GetOwner().GetOwner().GetColor( UVREditorMode::EColors::UISelectionBarColor ) );
	SetCloseButtonColor( GetOwner().GetOwner().GetColor( UVREditorMode::EColors::UICloseButtonColor ) );
}


void AVREditorDockableWindow::SetCollision(const ECollisionEnabled::Type InCollisionType, const ECollisionResponse InCollisionResponse, const ECollisionChannel InCollisionChannel)
{
	AVREditorFloatingUI::SetCollision(InCollisionType, InCollisionResponse, InCollisionChannel);

	SelectionBarMeshComponent->SetCollisionEnabled(InCollisionType);
	SelectionBarMeshComponent->SetCollisionResponseToAllChannels(InCollisionResponse);
	SelectionBarMeshComponent->SetCollisionObjectType(InCollisionChannel);
	CloseButtonMeshComponent->SetCollisionEnabled(InCollisionType);
	CloseButtonMeshComponent->SetCollisionResponseToAllChannels(InCollisionResponse);
}


void AVREditorDockableWindow::TickManually( float DeltaTime )
{	
	Super::TickManually( DeltaTime );

	if( WidgetComponent->IsVisible() )
	{
		const FVector2D Size = GetSize();
		const float WorldScaleFactor = GetOwner().GetOwner().GetWorldScaleFactor();
		const FVector AnimatedScale = CalculateAnimatedScale();

		const float CurrentScaleFactor = GetDockedTo() == EDockedTo::Nothing ? WorldPlacedScaleFactor : WorldScaleFactor;
		// Update whether the user is aiming toward us or not
		bIsLaserAimingTowardUI = false;

		if (!GetOwner().IsDraggingDockUI())
		{
			const FTransform UICapsuleTransform = this->GetActorTransform();

			const FVector UICapsuleStart = FVector( 0.0f, 0.0f, -Size.Y * 0.4f ) * CurrentScaleFactor * AnimatedScale;
			const FVector UICapsuleEnd = FVector( 0.0f, 0.0f, Size.Y * 0.4f ) * CurrentScaleFactor * AnimatedScale;
			const float UICapsuleLocalRadius = (float)Size.X * 0.5f * CurrentScaleFactor * (float)AnimatedScale.Y;
			const float MinDistanceToUICapsule = 10.0f * CurrentScaleFactor * (float)AnimatedScale.Y;	// @todo vreditor tweak
			const FVector UIForwardVector = FVector::ForwardVector;
			const float MinDotForAimingAtOtherHand = -1.1f;	// @todo vreditor tweak

			for( UViewportInteractor* Interactor : GetOwner().GetOwner().GetWorldInteraction().GetInteractors() )
			{
				if( GetOwner().GetOwner().IsHandAimingTowardsCapsule( Interactor, UICapsuleTransform, UICapsuleStart, UICapsuleEnd, UICapsuleLocalRadius, MinDistanceToUICapsule, UIForwardVector, MinDotForAimingAtOtherHand ) )
				{
					bIsLaserAimingTowardUI = true;
				}
			}

			if( bIsLaserAimingTowardUI )
			{
				AimingAtMeFadeAlpha += DeltaTime / VREd::DockUIFadeAnimationDuration->GetFloat();
			}
			else
			{
				AimingAtMeFadeAlpha -= DeltaTime / VREd::DockUIFadeAnimationDuration->GetFloat();;
			}
			AimingAtMeFadeAlpha = FMath::Clamp( AimingAtMeFadeAlpha, 0.0f, 1.0f );
		}

		const float AnimationOvershootAmount = 1.0f;	// @todo vreditor tweak
		float EasedAimingAtMeFadeAlpha = UVREditorMode::OvershootEaseOut( AimingAtMeFadeAlpha, AnimationOvershootAmount );

		// @todo: This should be refactored after Siggraph/Quokka. The VREditorSystem should spawn a VREditorFloatingUI if bHideWindowHandles is 

		// Handles to manipulate this window could be disabled. 		
		const bool bIsVisible = CreationContext.bHideWindowHandles ? false : (EasedAimingAtMeFadeAlpha > KINDA_SMALL_NUMBER ? true : false); // If enabled: only show our extra buttons and controls if the user is roughly aiming toward us to reduce visual clutter.
				
		DockButtonMeshComponent->SetVisibility(bIsVisible);
		SelectionBarMeshComponent->SetVisibility(bIsVisible);
		CloseButtonMeshComponent->SetVisibility(bIsVisible);				
		
		// If handles are diabled we don't want to collide with them, of course
		if (CreationContext.bHideWindowHandles)
		{
			DockButtonMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
			SelectionBarMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
			CloseButtonMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		}

		// The close button can be disabled 
		if (CreationContext.bNoCloseButton)
		{
			CloseButtonMeshComponent->SetVisibility(false);
			CloseButtonMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		}

		EasedAimingAtMeFadeAlpha = FMath::Max( 0.001f, EasedAimingAtMeFadeAlpha );

		// Update the dock button
		if (bIsHoveringOverDockButton)
		{
			DockButtonHoverAlpha += DeltaTime / VREd::DockUIHoverAnimationDuration->GetFloat();
		}
		else
		{
			DockButtonHoverAlpha -= DeltaTime / VREd::DockUIHoverAnimationDuration->GetFloat();;
		}
		DockButtonHoverAlpha = FMath::Clamp(DockButtonHoverAlpha, 0.0f, 1.0f);

		// How big the close button should be
		const FVector DockButtonSize(5.0f, Size.X * 0.12f, Size.X * 0.1f);
		FVector DockButtonScale = DockButtonSize * AnimatedScale * CurrentScaleFactor * EasedAimingAtMeFadeAlpha;
		DockButtonScale *= FMath::Lerp(1.0f, VREd::DockUIHoverScale->GetFloat(), DockButtonHoverAlpha);

		DockButtonMeshComponent->SetRelativeScale3D(DockButtonScale);
		const FVector DockButtonRelativeLocation = FVector(
			6.5f,
			((Size.X * 0.5f) - (DockButtonSize.Y * 0.5f)),
			-(Size.Y * 0.5f + (DockButtonSize.Z * 1.5f) + VREd::DockUISelectionBarVerticalOffset->GetFloat())) * AnimatedScale * CurrentScaleFactor;
		DockButtonMeshComponent->SetRelativeLocation(DockButtonRelativeLocation);

		SetDockButtonColor(GetOwner().GetOwner().GetColor(bIsHoveringOverDockButton ? UVREditorMode::EColors::UICloseButtonHoverColor : 
			GetDockedTo() == EDockedTo::Nothing ? UVREditorMode::EColors::SelectionColor : UVREditorMode::EColors::UICloseButtonColor));
		
		// Update the close button
		
		if (bIsHoveringOverCloseButton)
		{
			CloseButtonHoverAlpha += DeltaTime / VREd::DockUIHoverAnimationDuration->GetFloat();
		}
		else
		{
			CloseButtonHoverAlpha -= DeltaTime / VREd::DockUIHoverAnimationDuration->GetFloat();;
		}
		CloseButtonHoverAlpha = FMath::Clamp(CloseButtonHoverAlpha, 0.0f, 1.0f);

		// How big the close button should be
		const FVector CloseButtonSize(20.0f, Size.X * 0.1f, Size.X * 0.1f);
		FVector CloseButtonScale = CloseButtonSize * AnimatedScale * CurrentScaleFactor * EasedAimingAtMeFadeAlpha;
		CloseButtonScale *= FMath::Lerp(1.0f, VREd::DockUIHoverScale->GetFloat(), CloseButtonHoverAlpha);

		CloseButtonMeshComponent->SetRelativeScale3D(CloseButtonScale);
		const FVector CloseButtonRelativeLocation = FVector(
			4.0f,
			-((Size.X * 0.5f) - (CloseButtonSize.Y * 0.5f)),
			-(Size.Y * 0.5f + CloseButtonSize.Z + VREd::DockUISelectionBarVerticalOffset->GetFloat())) * AnimatedScale * CurrentScaleFactor;
		CloseButtonMeshComponent->SetRelativeLocation(CloseButtonRelativeLocation);

		SetCloseButtonColor(GetOwner().GetOwner().GetColor(bIsHoveringOverCloseButton ? UVREditorMode::EColors::UICloseButtonHoverColor : UVREditorMode::EColors::UICloseButtonColor));
		

		// Update the selection bar
		{
			if( bIsHoveringOverSelectionBar )
			{
				SelectionBarHoverAlpha += DeltaTime / VREd::DockUIHoverAnimationDuration->GetFloat();
			}
			else
			{
				SelectionBarHoverAlpha -= DeltaTime / VREd::DockUIHoverAnimationDuration->GetFloat();;
			}
			SelectionBarHoverAlpha = FMath::Clamp( SelectionBarHoverAlpha, 0.0f, 1.0f );
			
			// How big the selection bar should be
			float SelectionBarWidth = CreationContext.bNoCloseButton ? 0.82f : 0.7f;
			
			const FVector SelectionBarSize(20.0f, Size.X * SelectionBarWidth, Size.X * 0.1f);
			FVector SelectionBarScale = SelectionBarSize * AnimatedScale * CurrentScaleFactor;
			SelectionBarScale *= FMath::Lerp( 1.0f, VREd::DockUIHoverScale->GetFloat(), SelectionBarHoverAlpha );

			// Scale vertically based on our fade alpha
			SelectionBarScale.Z *= EasedAimingAtMeFadeAlpha;

			SelectionBarMeshComponent->SetRelativeScale3D( SelectionBarScale );
			const FVector SelectionBarRelativeLocation = FVector(
				4.0f,
				(Size.X * 0.5f - (SelectionBarSize.Y * 0.5f) - (1.5f * CloseButtonSize.Y)),
				-(Size.Y * 0.5f + SelectionBarSize.Z + VREd::DockUISelectionBarVerticalOffset->GetFloat())) * AnimatedScale * CurrentScaleFactor;
			SelectionBarMeshComponent->SetRelativeLocation( SelectionBarRelativeLocation );

			SetSelectionBarColor( GetOwner().GetOwner().GetColor( bIsHoveringOverSelectionBar ? UVREditorMode::EColors::UISelectionBarHoverColor : UVREditorMode::EColors::UISelectionBarColor ) );
		}

		
	}
} 

void AVREditorDockableWindow::UpdateRelativeRoomTransform()
{
	const FTransform RoomToWorld = GetOwner().GetOwner().GetRoomTransform();
	const FTransform WorldToRoom = RoomToWorld.Inverse();

	const FTransform WindowToWorldTransform = GetActorTransform();
	const FTransform WindowToRoomTransform = WindowToWorldTransform * WorldToRoom;

	const FVector RoomSpaceWindowLocation = WindowToRoomTransform.GetLocation() / GetOwner().GetOwner().GetWorldScaleFactor();;
	const FQuat RoomSpaceWindowRotation = WindowToRoomTransform.GetRotation();

	SetRelativeOffset( RoomSpaceWindowLocation );
	SetLocalRotation( RoomSpaceWindowRotation.Rotator() );
}

UStaticMeshComponent* AVREditorDockableWindow::GetCloseButtonMeshComponent() const
{
	return CloseButtonMeshComponent;
}

UStaticMeshComponent* AVREditorDockableWindow::GetSelectionBarMeshComponent() const
{
	return SelectionBarMeshComponent;
}

float AVREditorDockableWindow::GetDockSelectDistance() const
{
	return DockSelectDistance;
}

void AVREditorDockableWindow::SetDockSelectDistance(const float InDockDistance)
{
	DockSelectDistance = InDockDistance;
}

void AVREditorDockableWindow::OnPressed( UViewportInteractor* Interactor, const FHitResult& InHitResult, bool& bOutResultedInDrag )
{
	bOutResultedInDrag = false;

	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>( Interactor );
	if( VREditorInteractor != nullptr )
	{
		if( InHitResult.Component == GetCloseButtonMeshComponent() )
		{
			// Close the window
			const bool bShouldShow = false;
			const bool bSpawnInFront = false;
			GetOwner().ShowEditorUIPanel(this, VREditorInteractor, bShouldShow, bSpawnInFront);
		}
		if (InHitResult.Component == DockButtonMeshComponent)
		{
			if (GetDockedTo() == EDockedTo::Nothing)
			{
				SetDockedTo(EDockedTo::Room);
			}
			else if (GetDockedTo() == EDockedTo::Room)
			{
				SetDockedTo(EDockedTo::Nothing);
				WorldPlacedScaleFactor = GetOwner().GetOwner().GetWorldScaleFactor();
			}
		}
		else if(InHitResult.Component == GetSelectionBarMeshComponent() && !GetOwner().IsDraggingPanelFromOpen())
		{
			bOutResultedInDrag = true;
			SetDockSelectDistance((float)(InHitResult.TraceStart - InHitResult.Location ).Size());
			GetOwner().StartDraggingDockUI(this, VREditorInteractor, DockSelectDistance);
		}
	}
}

void AVREditorDockableWindow::OnHover( UViewportInteractor* Interactor )
{

}

void AVREditorDockableWindow::OnHoverEnter( UViewportInteractor* Interactor, const FHitResult& InHitResult )
{
	if ( SelectionBarMeshComponent == InHitResult.GetComponent() )
	{
		bIsHoveringOverSelectionBar = true;
	}

	if ( CloseButtonMeshComponent == InHitResult.GetComponent() )
	{
		bIsHoveringOverCloseButton = true;
	}

	if (DockButtonMeshComponent == InHitResult.GetComponent())
	{
		bIsHoveringOverDockButton = true;
	}
}

void AVREditorDockableWindow::OnHoverLeave( UViewportInteractor* Interactor, const UActorComponent* NewComponent )
{
	UActorComponent* OtherInteractorHoveredComponent = nullptr;
	if( Interactor->GetOtherInteractor() != nullptr )
	{
		OtherInteractorHoveredComponent = Interactor->GetOtherInteractor()->GetLastHoverComponent();
	}

	if ( OtherInteractorHoveredComponent != SelectionBarMeshComponent && NewComponent != SelectionBarMeshComponent && !GetDragOperationComponent()->IsDragging() )
	{
		bIsHoveringOverSelectionBar = false;
	}

	if ( OtherInteractorHoveredComponent != CloseButtonMeshComponent && NewComponent != CloseButtonMeshComponent )
	{
		bIsHoveringOverCloseButton = false;
	}

	if (OtherInteractorHoveredComponent != DockButtonMeshComponent && NewComponent != DockButtonMeshComponent)
	{
		bIsHoveringOverDockButton = false;
	}
}

void AVREditorDockableWindow::OnDragRelease( UViewportInteractor* Interactor )
{
	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>( Interactor );
	if( VREditorInteractor != nullptr )
	{
		UVREditorUISystem& UISystem = GetOwner();
		UISystem.StopDraggingDockUI( VREditorInteractor );
	}
}

UViewportDragOperationComponent* AVREditorDockableWindow::GetDragOperationComponent()
{
	return DragOperationComponent;
}

void AVREditorDockableWindow::SetWindowMesh(UStaticMesh* WindowMesh)
{	
	WindowMeshComponent->SetStaticMesh(WindowMesh);
	check(WindowMeshComponent != nullptr);
}

void AVREditorDockableWindow::SetSelectionBarColor( const FLinearColor& LinearColor )
{		
	static FName StaticColorParameterName( "Color" );
	SelectionBarMID->SetVectorParameterValue( StaticColorParameterName, LinearColor );
	SelectionBarTranslucentMID->SetVectorParameterValue( StaticColorParameterName, LinearColor );
}

void AVREditorDockableWindow::SetCloseButtonColor( const FLinearColor& LinearColor )
{
	static FName StaticColorParameterName( "Color" );
	CloseButtonMID->SetVectorParameterValue( StaticColorParameterName, LinearColor );
	CloseButtonTranslucentMID->SetVectorParameterValue( StaticColorParameterName, LinearColor );
}

void AVREditorDockableWindow::SetDockButtonColor(const FLinearColor& LinearColor)
{
	static FName StaticColorParameterName("Color");
	DockButtonMID->SetVectorParameterValue(StaticColorParameterName, LinearColor);
}

/************************************************************************/
/* Dockable window drag operation                                       */
/******************	******************************************************/
void UDockableWindowDragOperation::ExecuteDrag( UViewportInteractor* Interactor, IViewportInteractableInterface* Interactable  )
{
	// This is a "manual" interface Cast<> to avoid emitting a spurious deprecation warning.
	// This code will all be deleted in UE 5.9.
	if (!Interactable || !IsValid(Interactable->_getUObject()) || !Interactable->_getUObject()->IsA(AVREditorDockableWindow::StaticClass()))
	{
		return;
	}
	AVREditorDockableWindow* DockableWindow = (AVREditorDockableWindow*)Interactable->_getUObject();

	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>(Interactor);
	if ( VREditorInteractor && DockableWindow )
	{
		UVREditorUISystem& UISystem = DockableWindow->GetOwner();

		if (UISystem.CanScalePanel())
		{
			float NewUIScale = DockableWindow->GetScale() + VREditorInteractor->GetSlideDelta();
			if (NewUIScale <= UISystem.GetMinDockWindowSize())
			{
				NewUIScale = UISystem.GetMinDockWindowSize();
			}
			else if (NewUIScale >= UISystem.GetMaxDockWindowSize())
			{
				NewUIScale = UISystem.GetMaxDockWindowSize();
			}
			DockableWindow->SetScale(NewUIScale);
		}

		const FTransform UIToWorld = UISystem.MakeDockableUITransform( DockableWindow, VREditorInteractor, DockableWindow->GetDockSelectDistance() );
		FTransform SmoothedUIToWorld = UIToWorld;
		if( LastUIToWorld.IsSet() )
		{
			SmoothedUIToWorld.Blend( UIToWorld, LastUIToWorld.GetValue(), VREd::DockUIDragSmoothingAmount->GetFloat() );
		}

		// Update interactor hover location while dragging the interactor
		const FTransform LaserImpactToWorld = UISystem.MakeDockableUITransformOnLaser( DockableWindow, VREditorInteractor, DockableWindow->GetDockSelectDistance() );
		FTransform SmoothedLaserImpactToWorld = LaserImpactToWorld;
		if( LastLaserImpactToWorld.IsSet() )
		{
			SmoothedLaserImpactToWorld.Blend( LaserImpactToWorld, LastLaserImpactToWorld.GetValue(), VREd::DockUIDragSmoothingAmount->GetFloat() );
		}

		DockableWindow->SetActorTransform( SmoothedUIToWorld );
		DockableWindow->UpdateRelativeRoomTransform();

		Interactor->SetHoverLocation(SmoothedLaserImpactToWorld.GetLocation());

		LastUIToWorld = SmoothedUIToWorld;
		LastLaserImpactToWorld = SmoothedLaserImpactToWorld;
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
