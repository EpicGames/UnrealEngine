// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UI/VREditorFloatingUI.h"
#include "ViewportDragOperation.h"
#include "ViewportInteractableInterface.h"
#include "VREditorDockableWindow.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UStaticMeshComponent;
class UViewportInteractor;

/**
 * An interactive floating UI panel that can be dragged around
 */
UCLASS()
class AVREditorDockableWindow : public AVREditorFloatingUI, public IViewportInteractableInterface
{
	GENERATED_BODY()
	
public:

	/** Default constructor */
	AVREditorDockableWindow(const FObjectInitializer& ObjectInitializer);
	virtual void PostActorCreated() override;

	/** Updates the meshes for the UI */
	virtual void TickManually( float DeltaTime ) override;

	/** Updates the last dragged relative position */
	void UpdateRelativeRoomTransform();
	
	/** Gets the close button component */
	UStaticMeshComponent* GetCloseButtonMeshComponent() const;

	/** Gets the selection bar component */
	UStaticMeshComponent* GetSelectionBarMeshComponent() const;

	/** Gets the distance between the interactor and the window when starting drag */
	float GetDockSelectDistance() const;

	/** Set the distance between the interactor and the window when starting drag */
	void SetDockSelectDistance(const float InDockDistance);

	// IViewportInteractableInterface
	virtual void OnPressed( UViewportInteractor* Interactor, const FHitResult& InHitResult, bool& bOutResultedInDrag ) override;
	virtual void OnHover( UViewportInteractor* Interactor ) override;
	
	/** Enter hover with laser changes the color of SelectionMesh and CloseButtonMesh */
	virtual void OnHoverEnter( UViewportInteractor* Interactor, const FHitResult& InHitResult ) override;
	
	/** Leaving hover with laser changes the color of SelectionMesh and CloseButtonMesh */
	virtual void OnHoverLeave( UViewportInteractor* Interactor, const UActorComponent* NewComponent ) override;
	virtual void OnDragRelease( UViewportInteractor* Interactor ) override;
	virtual class UViewportDragOperationComponent* GetDragOperationComponent() override;
	virtual bool CanBeSelected() override { return false; };

protected:

	// AVREditorFloatingUI overrides
	virtual void SetupWidgetComponent() override;
	virtual void SetCollision(const ECollisionEnabled::Type InCollisionType, const ECollisionResponse InCollisionResponse, const ECollisionChannel InCollisionChannel) override;

private:

	/** Set the static mesh for the main window that backs our content */
	void SetWindowMesh(UStaticMesh* WindowMesh);

	/** Set the color on the dynamic materials of the selection bar */
	void SetSelectionBarColor( const FLinearColor& LinearColor );

	/** Set the color on the dynamic materials of the close button */
	void SetCloseButtonColor( const FLinearColor& LinearColor );

	/** Set the color on the dynamic materials of the dock button */
	void SetDockButtonColor(const FLinearColor& LinearColor);

	/** Mesh underneath the window for docking it to the world */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> DockButtonMeshComponent;

	/** Mesh underneath the window for easy selecting and dragging */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> SelectionBarMeshComponent;

	/** Mesh that represents the close button for this UI */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> CloseButtonMeshComponent;

	/** Dock Button dynamic material  (opaque) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> DockButtonMID;

	/** Selection bar dynamic material  (opaque) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> SelectionBarMID;

	/** Select bar dynamic material (translucent) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> SelectionBarTranslucentMID;

	/** Close button dynamic material  (opaque) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> CloseButtonMID;

	/** Close button dynamic material (translucent) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> CloseButtonTranslucentMID;

	UPROPERTY()
	TObjectPtr<class UViewportDragOperationComponent> DragOperationComponent;

	/** True if at least one hand's laser is aiming toward the UI */
	bool bIsLaserAimingTowardUI;

	/** Scalar that ramps up toward 1.0 after the user aims toward the UI */
	float AimingAtMeFadeAlpha;

	/** True if we're hovering over the selection bar */
	bool bIsHoveringOverSelectionBar;

	/** Scalar that will advance toward 1.0 over time as we hover over the selection bar */
	float SelectionBarHoverAlpha;

	/** True if we're hovering over the close button */
	bool bIsHoveringOverCloseButton;

	/** True if we're hovering over the dock button */
	bool bIsHoveringOverDockButton;

	/** Scalar that will advance toward 1.0 over time as we hover over the close button */
	float CloseButtonHoverAlpha;

	/** Scalar that will advance toward 1.0 over time as we hover over the dock button */
	float DockButtonHoverAlpha;

	/** Distance from interactor laser to the handle when starting dragging */
	float DockSelectDistance;

};

/**
 * Calculation for dragging a dockable window
 */
UCLASS()
class UDockableWindowDragOperation : public UViewportDragOperation
{
	GENERATED_BODY()

public:

	// IViewportDragOperation
	virtual void ExecuteDrag( class UViewportInteractor* Interactor, IViewportInteractableInterface* Interactable ) override;

	/** Last frame's UIToWorld transform */
	TOptional<FTransform> LastUIToWorld;

	/** last frame's LaserImpactToWorld transform */
	TOptional<FTransform> LastLaserImpactToWorld;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
