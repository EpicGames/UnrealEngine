// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorFloatingCameraUI.h"
#include "UI/VREditorUISystem.h"
#include "UI/VREditorBaseUserWidget.h"
#include "VREditorMode.h"
#include "VREditorCameraWidgetComponent.h"
#include "Components/StaticMeshComponent.h"
#include "VRModeSettings.h"
#include "VREditorAssetContainer.h"
#include "SLevelViewport.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorFloatingCameraUI)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "AVREditorFloatingCameraUI"

AVREditorFloatingCameraUI::AVREditorFloatingCameraUI(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UVREditorCameraWidgetComponent>(TEXT("WidgetComponent"))),
	OffsetFromCamera( -25.0f, 0.0f, 30.0f )
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	{
		const UVREditorAssetContainer& AssetContainer = UVREditorMode::LoadAssetContainer();
		UStaticMesh* WindowMesh = AssetContainer.WindowMesh;
		WindowMeshComponent->SetStaticMesh(WindowMesh);
		check(WindowMeshComponent != nullptr);
	}
}

void AVREditorFloatingCameraUI::SetLinkedActor(class AActor* InActor)
{
	LinkedActor = InActor;
}

FTransform AVREditorFloatingCameraUI::MakeCustomUITransform()
{
	FTransform CameraTransform = FTransform::Identity;
	FTransform UITransform = FTransform::Identity;
	if (LinkedActor != nullptr)
	{
		CameraTransform = LinkedActor->GetTransform();

		const FTransform UIFlipTransform(FRotator(0.0f, 180.0f, 0.0f).Quaternion(), FVector::ZeroVector);
		const FTransform OffsetTransform(FRotator::ZeroRotator, OffsetFromCamera);

		UITransform = UIFlipTransform * OffsetTransform * CameraTransform;
	}
	return UITransform;
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
