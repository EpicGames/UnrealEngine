// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "UI/VREditorFloatingUI.h"
#include "VREditorFloatingCameraUI.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UVREditorBaseUserWidget;
class UVREditorUISystem;

typedef FName VREditorPanelID;

/**
 * Represents an interactive floating UI camera preview panel in the VR Editor
 */
UCLASS(Transient)
class AVREditorFloatingCameraUI : public AVREditorFloatingUI
{
	GENERATED_BODY()

public:

	/** The offset of this UI from its camera */
	UPROPERTY(EditDefaultsOnly, Category = "FloatingCameraUI")
	FVector OffsetFromCamera;

	AVREditorFloatingCameraUI(const FObjectInitializer& ObjectInitializer);
	void SetLinkedActor(class AActor* InActor);

	virtual FTransform MakeCustomUITransform() override;

private:
	UPROPERTY( )
	TWeakObjectPtr<AActor> LinkedActor;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
