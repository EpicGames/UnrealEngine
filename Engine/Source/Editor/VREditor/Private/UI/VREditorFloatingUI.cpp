// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorFloatingUI.h"
#include "UI/VREditorUISystem.h"
#include "UI/VREditorBaseUserWidget.h"
#include "VREditorMode.h"
#include "Components/WidgetComponent.h"
#include "VREditorWidgetComponent.h"
#include "Components/StaticMeshComponent.h"
#include "VRModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorFloatingUI)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable UIFadeSpeed(TEXT("VREd.UIFadeSpeed"), 6.0f, TEXT("How fast UI should fade in and out"));
}

AVREditorFloatingUI::AVREditorFloatingUI(const FObjectInitializer& ObjectInitializer)
	: Super(),
	SlateWidget(nullptr),
	UserWidget(nullptr),
	WidgetComponent(nullptr),
	Resolution(0, 0),
	Owner(nullptr),
	UserWidgetClass(nullptr),
	bShouldBeVisible(),
	FadeAlpha( 1.0f ),
	FadeDelay( 0.0f ),
	InitialScale( 1.0f ),
	UISystemID(NAME_None),
	bClearWidgetOnHide(false)
{
	const bool bTransient = true;
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"), bTransient);
	check(SceneComponent != nullptr);
	this->RootComponent = SceneComponent;

	WidgetComponent = CreateDefaultSubobject<UVREditorWidgetComponent>(TEXT("WidgetComponent"), bTransient);
	WidgetComponent->SetEditTimeUsable(true);
	WidgetComponent->SetupAttachment(SceneComponent);
	WidgetComponent->PrimaryComponentTick.bTickEvenWhenPaused = true;
	InitialScale = Scale;

	{
		WindowMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("WindowMesh"));
		WindowMeshComponent->SetMobility(EComponentMobility::Movable);
		WindowMeshComponent->SetupAttachment(RootComponent);

		WindowMeshComponent->SetGenerateOverlapEvents(false);
		WindowMeshComponent->SetCanEverAffectNavigation(false);
		WindowMeshComponent->bCastDynamicShadow = false;
		WindowMeshComponent->bCastStaticShadow = false;
		WindowMeshComponent->bAffectDistanceFieldLighting = false;
		WindowMeshComponent->bSelectable = false;
	}
}


void AVREditorFloatingUI::SetupWidgetComponent()
{
	WidgetComponent->SetTwoSided(false);	// No VR UI is two-sided

	if (SlateWidget.IsValid())
	{
		// Slate UIs have bogus opacity in their texture's alpha, so ignore texture alpha for VR
		WidgetComponent->SetOpacityFromTexture(0.0f);	// Slate UIs have bogus opacity in their texture's alpha, so ignore texture alpha for VR
		WidgetComponent->SetBackgroundColor(FLinearColor::Black);
		WidgetComponent->SetBlendMode(EWidgetBlendMode::Opaque);
	}
	else // UMG UIs
	{
		if (!CreationContext.bMaskOutWidgetBackground) // Default behavior
		{
			WidgetComponent->SetOpacityFromTexture(1.0f);
			WidgetComponent->SetBackgroundColor(FLinearColor::Transparent);
			WidgetComponent->SetBlendMode(EWidgetBlendMode::Masked);
		}
		else // User override via CreationContext
		{
			WidgetComponent->SetOpacityFromTexture(1.0f);
			WidgetComponent->SetBackgroundColor(FLinearColor::Transparent);
			WidgetComponent->SetBlendMode(EWidgetBlendMode::Transparent);			
		}
	}

	// @todo vreditor: Ideally we use automatic mip map generation, otherwise the UI looks too crunchy at a distance.
	// However, I tried this and on D3D11 the mips are all black.
	WidgetComponent->SetDrawSize(FVector2D(Resolution.X, Resolution.Y));		// NOTE: Must be called before RegisterComponent() because collision data will be created during registration

	// NOTE: Must be called *after* RegisterComponent() because UWidgetComponent nulls out Widget if no WidgetClass is set (WidgetClass is protected and there is no accessor)
	if (SlateWidget.IsValid())
	{
		WidgetComponent->SetSlateWidget(SlateWidget.ToSharedRef());
	}
	else if (UserWidgetClass != nullptr)
	{
		// @todo vreditor unreal: Ideally we would do this in the constructor and not again after.  Due to an apparent bug in UMG, 
		// we need to re-create the widget in lock-step with the WidgetComponent, otherwise input doesn't function correctly on the 
		// widget after the widget component is destroyed and recreated with the same user widget.
		UserWidget = CreateWidget<UUserWidget>(GetWorld(), UserWidgetClass);
		check(UserWidget != nullptr);

		WidgetComponent->SetWidget(UserWidget);
	}

	// @todo vreditor: Is this useful?
	//WidgetComponent->SetMaxInteractionDistance( 10000.0f );

	// Default to visible
	ShowUI(false);

	// Set initial opacity
	UpdateFadingState(0.0f);

	// Set initial transform
	UpdateTransformIfDocked();

	// Update the window border mesh
	{
		const float WindowMeshSize = 100.0f;	// Size of imported mesh, we need to inverse compensate for

		const FVector WindowMeshScale = FVector(
			1.0f,
			GetSize().X / WindowMeshSize,
			GetSize().Y / WindowMeshSize) * GetOwner().GetOwner().GetWorldScaleFactor();
		WindowMeshComponent->SetRelativeScale3D(WindowMeshScale);
	}
}

void AVREditorFloatingUI::SetSlateWidget( UVREditorUISystem& InitOwner, const VREditorPanelID& InID, const TSharedRef<SWidget>& InitSlateWidget, const FIntPoint InitResolution, const float InitScale, const EDockedTo InitDockedTo )
{
	Owner = &InitOwner;
	SetVRMode(&Owner->GetOwner());

	UISystemID = InID;

	SlateWidget = InitSlateWidget;

	Resolution = InitResolution;
	check(Resolution.X > 0 && Resolution.Y > 0);

	Scale = InitScale;
	InitialScale = Scale;

	SetDockedTo(InitDockedTo);
	SetupWidgetComponent();
}

void AVREditorFloatingUI::SetSlateWidget(const TSharedRef<SWidget>& InitSlateWidget)
{
	SlateWidget = InitSlateWidget;
	SetupWidgetComponent();
}

void AVREditorFloatingUI::SetUMGWidget( UVREditorUISystem& InitOwner, const VREditorPanelID& InID, TSubclassOf<UUserWidget> InitUserWidgetClass, const FIntPoint InitResolution, const float InitScale, const EDockedTo InitDockedTo )
{
	Owner = &InitOwner;
	SetVRMode(&Owner->GetOwner());

	UISystemID = InID;

	UserWidgetClass = InitUserWidgetClass;

	Resolution = InitResolution;
	check(Resolution.X > 0 && Resolution.Y > 0);

	Scale = InitScale;
	InitialScale = Scale;

	SetDockedTo(InitDockedTo);

	SetupWidgetComponent();
}

void AVREditorFloatingUI::TickManually(float DeltaTime)
{
	Super::TickManually(DeltaTime);

	if (WindowMeshComponent != nullptr && WidgetComponent->IsVisible())
	{
		const float WorldScaleFactor = GetOwner().GetOwner().GetWorldScaleFactor();

		const float CurrentScaleFactor = (GetDockedTo() == EDockedTo::Nothing && WorldPlacedScaleFactor != 0) ? WorldPlacedScaleFactor : WorldScaleFactor;

		const FVector AnimatedScale = CalculateAnimatedScale();
		const FVector2D Size = GetSize();
		const float WindowMeshSize = 100.0f;	// Size of imported mesh, we need to inverse compensate for

		const FVector WindowMeshScale = FVector(1.0,
			Size.X / WindowMeshSize,
			Size.Y / WindowMeshSize) * AnimatedScale * CurrentScaleFactor;
		WindowMeshComponent->SetRelativeScale3D(WindowMeshScale);

		const FVector NewScale(GetScale() * AnimatedScale * CurrentScaleFactor);
		SetWidgetComponentScale(NewScale);
	}
}

void AVREditorFloatingUI::Destroyed()
{
	if (IsValidChecked(this))
	{
		CleanupWidgetReferences();
	}

	Super::Destroyed();
}

void AVREditorFloatingUI::CleanupWidgetReferences()
{
	if (WidgetComponent != nullptr)
	{
		// NOTE: We're nulling out widgets so that we don't have to wait for a GC to free up Slate resources (avoid shutdown crash)
		WidgetComponent->SetSlateWidget(nullptr);
		WidgetComponent->SetWidget(nullptr);
		WidgetComponent = nullptr;
	}

	if (SlateWidget.IsValid())
	{
		SlateWidget.Reset();
	}

	// @todo vreditor unreal: UMG has a bug that prevents you from re-using the user widget for a new widget component
	// after a previous widget component that was using it was destroyed
	if (UserWidget != nullptr)
	{
		UserWidget->MarkAsGarbage();
		UserWidget = nullptr;
	}
}

void AVREditorFloatingUI::SetTransform(const FTransform& Transform)
{
	if (!IsHidden())
	{
		const FVector AnimatedScale = CalculateAnimatedScale();
		FTransform AnimatedTransform = Transform;
		AnimatedTransform.SetScale3D(AnimatedTransform.GetScale3D() * AnimatedScale);

		RootComponent->SetWorldLocation(AnimatedTransform.GetLocation());
		RootComponent->SetWorldRotation(AnimatedTransform.GetRotation());

		SetWidgetComponentScale(AnimatedTransform.GetScale3D());
	}
}


void AVREditorFloatingUI::BeginDestroy()
{
	CleanupWidgetReferences();

	Super::BeginDestroy();
}

void AVREditorFloatingUI::UpdateFadingState(const float DeltaTime)
{
	if (FadeDelay > 0.f)
	{
		FadeDelay -= DeltaTime;
	}
	else
	{
		if (bShouldBeVisible.GetValue())
		{
			FadeAlpha += VREd::UIFadeSpeed->GetFloat() * DeltaTime;
		}
		else
		{
			FadeAlpha -= VREd::UIFadeSpeed->GetFloat() * DeltaTime;
		}
		FadeAlpha = FMath::Clamp(FadeAlpha, 0.0f, 1.0f);

		if (FadeAlpha > 0.0f + KINDA_SMALL_NUMBER)
		{
			// At least a little bit visible
			if (IsHidden())
			{
				SetHidden(false);

				// Iterate as floating UI children may have other mesh components
				TInlineComponentArray<USceneComponent*> ComponentArray;
				GetComponents(ComponentArray);
				for (USceneComponent* Component : ComponentArray)
				{
					Component->SetVisibility(true);
				}
				FadeDelay = 0.0f;
			}
		}

		if (FadeAlpha >= 1.0f - KINDA_SMALL_NUMBER)
		{
			// Fully visible
		}
		else if (FadeAlpha <= 0.0f + KINDA_SMALL_NUMBER)
		{
			// Fully invisible
			if (!IsHidden())
			{
				SetHidden(true);
				// Iterate as floating UI children may have other mesh components
				TInlineComponentArray<USceneComponent*> ComponentArray;
				GetComponents(ComponentArray);
				for (USceneComponent* Component : ComponentArray)
				{
					Component->SetVisibility(false);
				}
				FadeDelay = 0.0f;

				if (bClearWidgetOnHide)
				{
					SetSlateWidget(SNullWidget::NullWidget);
					bClearWidgetOnHide = false;
				}
			}
		}

		// Set material color
		const float UIBrightness = FadeAlpha * GetDefault<UVRModeSettings>()->UIBrightness;
		WidgetComponent->SetTintColorAndOpacity(FLinearColor(UIBrightness, UIBrightness, UIBrightness).CopyWithNewOpacity(FadeAlpha));

	}
}

FVector AVREditorFloatingUI::CalculateAnimatedScale() const
{
	const float AnimationOvershootAmount = 0.7f;	// @todo vreditor tweak
	float EasedAlpha = UVREditorMode::OvershootEaseOut(FadeAlpha, AnimationOvershootAmount);
	EasedAlpha = FMath::Clamp(EasedAlpha, 0.01f, 1.0f + AnimationOvershootAmount);

	// Animate vertically more than horizontally; just looks a little better
	const float ZScale = FMath::Max(0.001f, EasedAlpha);
	const float YScale = FMath::Max(0.001f, 0.7f + 0.3f * EasedAlpha);

	FVector AnimatedScale = FVector(1.0f, YScale, ZScale);
	AnimatedScale.Y *= YScale;
	AnimatedScale.Z *= ZScale;

	return AnimatedScale;
}

void AVREditorFloatingUI::SetCollision(const ECollisionEnabled::Type InCollisionType, const ECollisionResponse InCollisionResponse, const ECollisionChannel InCollisionChannel)
{
	WidgetComponent->SetCollisionEnabled(InCollisionType);
	WidgetComponent->SetCollisionResponseToAllChannels(InCollisionResponse);
	WidgetComponent->SetCollisionObjectType(InCollisionChannel);

	if (WindowMeshComponent)
	{
		WindowMeshComponent->SetCollisionEnabled(InCollisionType);
		WindowMeshComponent->SetCollisionResponseToAllChannels(InCollisionResponse);
		WindowMeshComponent->SetCollisionObjectType(InCollisionChannel);
	}
}

UUserWidget* AVREditorFloatingUI::GetUserWidget()
{
	return UserWidget;
}

float AVREditorFloatingUI::GetInitialScale() const
{
	return InitialScale;
}

void AVREditorFloatingUI::ShowUI( const bool bShow, const bool bAllowFading, const float InitFadeDelay, const bool bInClearWidgetOnHide )
{
	if (!bShouldBeVisible.IsSet() || bShow != bShouldBeVisible.GetValue())
	{
		bShouldBeVisible = bShow;

		if (!bAllowFading)
		{
			SetHidden(!bShow);
			// Iterate as floating UI children may have other components
			TInlineComponentArray<USceneComponent*> ComponentArray;
			GetComponents(ComponentArray);
			for (USceneComponent* Component : ComponentArray)
			{
				Component->SetVisibility(bShow);
			}
			FadeAlpha = bShow ? 1.0f : 0.0f;
			if (bInClearWidgetOnHide )
			{
				SetSlateWidget(SNullWidget::NullWidget);
			}
		}
		else
		{
			if (bInClearWidgetOnHide)
			{
				bClearWidgetOnHide = bInClearWidgetOnHide;
			}
		}
		


		// Set collision on components
		if (bShow)
		{
			SetCollision(ECollisionEnabled::QueryOnly, ECollisionResponse::ECR_Block, ECollisionChannel::ECC_WorldStatic);
		}
		else
		{
			SetCollision(ECollisionEnabled::NoCollision, ECollisionResponse::ECR_Ignore, ECollisionChannel::ECC_Visibility);
		}


		FadeDelay = InitFadeDelay;
	}
}


void AVREditorFloatingUI::SetResolution(const FIntPoint& InResolution)
{
	Resolution = InResolution;
	check(Resolution.X > 0 && Resolution.Y > 0);

	WidgetComponent->SetDrawSize(FVector2D(Resolution.X, Resolution.Y));		// NOTE: Must be called before 
	
	{
		const float WindowMeshSize = 100.0f;	// Size of imported mesh, we need to inverse compensate for

		const FVector WindowMeshScale = FVector(
			1.0f,
			GetSize().X / WindowMeshSize,
			GetSize().Y / WindowMeshSize) * GetOwner().GetOwner().GetWorldScaleFactor();
		WindowMeshComponent->SetRelativeScale3D(WindowMeshScale);
	}
}

FVector2D AVREditorFloatingUI::GetSize() const
{
	const float Aspect = (float)Resolution.X / (float)Resolution.Y;
	return FVector2D(Scale, Scale / Aspect);
}

float AVREditorFloatingUI::GetScale() const
{
	return Scale;
}

void AVREditorFloatingUI::SetScale(const float NewSize, const bool bScaleWidget /*= true*/)
{
	Scale = NewSize;

	if (bScaleWidget)
	{
		const float WorldScaleFactor = Owner->GetOwner().GetWorldScaleFactor();
		const FVector NewScale( Scale * WorldScaleFactor );
		SetWidgetComponentScale(NewScale);
	}
}

void AVREditorFloatingUI::SetWidgetComponentScale(const FVector& InScale)
{
	const float Aspect = (float)Resolution.X / (float)Resolution.Y;
	WidgetComponent->SetWorldScale3D(FVector(1.0f / InScale.X, 1.0f / (float)Resolution.X, 1.0f / (float)Resolution.Y / Aspect) * InScale);
}

VREditorPanelID AVREditorFloatingUI::GetID() const
{
	return UISystemID;
}

TSharedPtr<SWidget> AVREditorFloatingUI::GetSlateWidget() const
{
	return SlateWidget;
}

void AVREditorFloatingUI::SetWindowMesh(class UStaticMesh* InWindowMesh)
{
	check(InWindowMesh != nullptr);
	WindowMeshComponent->SetStaticMesh(InWindowMesh);
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
