// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorRadialFloatingUI.h"
#include "UI/VREditorUISystem.h"
#include "UI/VREditorBaseUserWidget.h"
#include "VREditorMode.h"
#include "Components/WidgetComponent.h"
#include "VREditorWidgetComponent.h"
#include "Components/StaticMeshComponent.h"
#include "VREditorActions.h"
#include "Framework/Application/SlateApplication.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "VREditorAssetContainer.h"
#include "VRModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorRadialFloatingUI)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable RadialUIFadeSpeed( TEXT( "VREd.RadialUIFadeSpeed" ), 6.0f, TEXT( "How fast UI should fade in and out" ) );
	static FAutoConsoleVariable RadialUIBrightness( TEXT( "VREd.RadialUIBrightness" ), 1.5f, TEXT( "How bright the UI should be" ) );
	static FAutoConsoleVariable MinJoystickOffsetBeforeRadialMenu(TEXT("VREd.MinJoystickOffsetBeforeRadialMenu"), 0.4f, TEXT("Toggles inverting the touch pad vertical axis"));
	static FAutoConsoleVariable CentralWidgetX(TEXT("VREd.CentralWidgetX"), 512, TEXT("Horizontal resolution to use for VR editor radial UI render targets"));
	static FAutoConsoleVariable CentralWidgetY(TEXT("VREd.CentralWidgetY"), 512, TEXT("Vertical resolution to use for VR editor radial UI render targets"));

}


AVREditorRadialFloatingUI::AVREditorRadialFloatingUI()
	: Super(),
	Resolution( 0, 0 ),
	Owner( nullptr ),
	bShouldBeVisible(),
	FadeAlpha( 1.0f ),
	FadeDelay( 0.0f ),
	InitialScale( 1.0f )
{
	const bool bTransient = true;
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"), bTransient);
	check(SceneComponent != nullptr);
	this->RootComponent = SceneComponent;

	DefaultGlowAmount = 2.0f;
}


void AVREditorRadialFloatingUI::PostActorCreated()
{
	Super::PostActorCreated();

	const UVREditorAssetContainer& AssetContainer = UVREditorMode::LoadAssetContainer();

	{
		WindowMeshComponent = NewObject<UStaticMeshComponent>(this, TEXT("WindowMesh"));
		WindowMeshComponent->SetMobility(EComponentMobility::Movable);
		WindowMeshComponent->SetupAttachment(RootComponent);
		WindowMeshComponent->RegisterComponent();
		WindowMeshComponent->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
		WindowMeshComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);

		WindowMeshComponent->SetStaticMesh(AssetContainer.RadialMenuMainMesh);
		WindowMeshComponent->CreateAndSetMaterialInstanceDynamic(0);
		UMaterialInstanceDynamic* DiskMaterial = Cast<UMaterialInstanceDynamic>(WindowMeshComponent->GetMaterial(0));
		GlowAmount = DefaultGlowAmount;
		DiskMaterial->SetScalarParameterValue("GlowAmount", GlowAmount);
		WindowMeshComponent->SetRelativeLocation(FVector(-4.0f, 0.0f, 0.0f));
		WindowMeshComponent->SetRelativeRotation(FRotator(-90.0f, 0.0f, 0.0f).Quaternion());
		WindowMeshComponent->SetRelativeScale3D(FVector(2.5f));

		WindowMeshComponent->SetGenerateOverlapEvents(false);
		WindowMeshComponent->SetCanEverAffectNavigation(false);
		WindowMeshComponent->bCastDynamicShadow = false;
		WindowMeshComponent->bCastStaticShadow = false;
		WindowMeshComponent->bAffectDistanceFieldLighting = false;
		WindowMeshComponent->bSelectable = false;
	}

	{
		ArrowMeshComponent = NewObject<UStaticMeshComponent>(this, TEXT("ArrowMesh"));
		ArrowMeshComponent->SetMobility(EComponentMobility::Movable);
		ArrowMeshComponent->SetupAttachment(WindowMeshComponent);
		ArrowMeshComponent->RegisterComponent();
		ArrowMeshComponent->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
		ArrowMeshComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);

		ArrowMeshComponent->SetStaticMesh(AssetContainer.RadialMenuPointerMesh);
		ArrowMeshComponent->CreateAndSetMaterialInstanceDynamic(0);
		UMaterialInstanceDynamic* ArrowMaterial = Cast<UMaterialInstanceDynamic>(ArrowMeshComponent->GetMaterial(0));
		ArrowAlpha = 0.0f;
		ArrowMaterial->SetScalarParameterValue("Alpha", ArrowAlpha);
		ArrowMeshComponent->SetGenerateOverlapEvents(false);
		ArrowMeshComponent->SetCanEverAffectNavigation(false);
		ArrowMeshComponent->bCastDynamicShadow = false;
		ArrowMeshComponent->bCastStaticShadow = false;
		ArrowMeshComponent->bAffectDistanceFieldLighting = false;
		ArrowMeshComponent->bSelectable = false;
		ArrowMeshComponent->SetVisibility(false);
	}

	{
		CentralWidgetComponent = NewObject<UVREditorWidgetComponent>(this, TEXT("CentralWidget"));
		CentralWidgetComponent->SetupAttachment(RootComponent);
		CentralWidgetComponent->RegisterComponent();
		CentralWidgetComponent->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
		CentralWidgetComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);
		CentralWidgetComponent->SetGenerateOverlapEvents(false);
		CentralWidgetComponent->SetCanEverAffectNavigation(false);
		CentralWidgetComponent->bCastDynamicShadow = false;
		CentralWidgetComponent->bCastStaticShadow = false;
		CentralWidgetComponent->bAffectDistanceFieldLighting = false;
		CentralWidgetComponent->bSelectable = false;
		CentralWidgetComponent->PrimaryComponentTick.bTickEvenWhenPaused = true;
		if (CentralSlateWidget.IsValid())
		{
			CentralWidgetComponent->SetSlateWidget(CentralSlateWidget);
		}
		CentralWidgetComponent->SetEditTimeUsable(true);
		CentralWidgetComponent->SetTwoSided(false);	// No VR UI is two-sided
		CentralWidgetComponent->SetOpacityFromTexture(1.0f);	// Slate UIs have bogus opacity in their texture's alpha, so ignore texture alpha for VR
		CentralWidgetComponent->SetBackgroundColor(FLinearColor::Transparent);
		CentralWidgetComponent->SetBlendMode(EWidgetBlendMode::Transparent);
		CentralWidgetComponent->SetDrawSize(FVector2D(VREd::CentralWidgetX->GetFloat(), VREd::CentralWidgetY->GetFloat()));
	}
}

void AVREditorRadialFloatingUI::SetupWidgetComponent(TSharedPtr<SWidget> SlateWidget)
{

	const float WorldScaleFactor = GetOwner().GetOwner().GetWorldScaleFactor();

	UVREditorWidgetComponent* NewWidgetComponent = NewObject<UVREditorWidgetComponent>(this);
	NewWidgetComponent->SetEditTimeUsable(true);
	NewWidgetComponent->SetupAttachment(RootComponent);
	NewWidgetComponent->PrimaryComponentTick.bTickEvenWhenPaused = true;
	AddOwnedComponent(NewWidgetComponent);
	NewWidgetComponent->RegisterComponent();

	InitialScale = Scale;
	NewWidgetComponent->SetTwoSided( false );	// No VR UI is two-sided
	NewWidgetComponent->SetOpacityFromTexture( 1.0f );	// Slate UIs have bogus opacity in their texture's alpha, so ignore texture alpha for VR
	NewWidgetComponent->SetBackgroundColor( FLinearColor::Transparent );
	NewWidgetComponent->SetBlendMode( EWidgetBlendMode::Masked );

	const int32 PositionSoFar = WidgetComponents.Num();
	const float Radius = 14.0f * WorldScaleFactor;
	float RadialX = -Radius * FMath::Sin(PositionSoFar * ((2.0f*PI) / NumberOfEntries));
	float RadialY = Radius * FMath::Cos(PositionSoFar * ((2.0f*PI) / NumberOfEntries));
	NewWidgetComponent->SetRelativeScale3D(FVector(1.0f / 25.0f) * WorldScaleFactor);
	NewWidgetComponent->SetRelativeLocation(FVector(0.0f, RadialX, RadialY));
	
	// @todo vreditor: Ideally we use automatic mip map generation, otherwise the UI looks too crunchy at a distance.
	// However, I tried this and on D3D11 the mips are all black.
	NewWidgetComponent->SetDrawSize( FVector2D(Resolution.X, Resolution.Y ) );		// NOTE: Must be called before RegisterComponent() because collision data will be created during registration

	// NOTE: Must be called *after* RegisterComponent() because UWidgetComponent nulls out Widget if no WidgetClass is set (WidgetClass is protected and there is no accessor)
	if( SlateWidget.IsValid() )
	{
		NewWidgetComponent->SetSlateWidget( SlateWidget.ToSharedRef() );
	}

	WidgetComponents.Add(NewWidgetComponent);

}

void AVREditorRadialFloatingUI::Reset()
{
	for (UVREditorWidgetComponent* WidgetComponent : WidgetComponents)
	{
		if (WidgetComponent != nullptr)
		{
			// NOTE: We're nulling out widgets so that we don't have to wait for a GC to free up Slate resources (avoid shutdown crash)
			WidgetComponent->SetSlateWidget(nullptr);
			WidgetComponent->DestroyComponent();
		}
	}

	for (TSharedPtr<SWidget> &SlateWidget : SlateWidgets)
	{
		SlateWidget = nullptr;
	}
	WidgetComponents.Empty();
	SlateWidgets.Empty();
}

void AVREditorRadialFloatingUI::SetSlateWidget(UVREditorUISystem& InitOwner, const TSharedRef<SWidget>& InitSlateWidget, const FIntPoint InitResolution, const float InitScale, const EDockedTo InitDockedTo)
{
	Owner = &InitOwner;
	SetVRMode( &Owner->GetOwner() );

	InitSlateWidget->SetVisibility(EVisibility::Visible);

	SlateWidgets.Add(InitSlateWidget);

	Resolution = InitResolution;
	check( Resolution.X > 0 && Resolution.Y > 0 );

	Scale = InitScale;
	InitialScale = Scale;

	SetDockedTo( InitDockedTo );

	SetupWidgetComponent(InitSlateWidget);
}

void AVREditorRadialFloatingUI::Destroyed()
{
	for (UVREditorWidgetComponent* WidgetComponent : WidgetComponents)
	{
		if (WidgetComponent != nullptr)
		{
			// NOTE: We're nulling out widgets so that we don't have to wait for a GC to free up Slate resources (avoid shutdown crash)
			WidgetComponent->SetSlateWidget(nullptr);
			WidgetComponent = nullptr;
		}
	}

	CentralWidgetComponent->SetSlateWidget(nullptr);
	CentralWidgetComponent = nullptr;

	for(TSharedPtr<SWidget> &SlateWidget : SlateWidgets)
	{
		SlateWidget = nullptr;
	}
	CentralSlateWidget = nullptr;

	Super::Destroyed();
}

void AVREditorRadialFloatingUI::SetTransform( const FTransform& Transform )
{
	const FVector AnimatedScale = CalculateAnimatedScale();
	const float WorldScaleFactor = GetOwner().GetOwner().GetWorldScaleFactor();
	FTransform AnimatedTransform = Transform;
	AnimatedTransform.SetScale3D( AnimatedTransform.GetScale3D() * AnimatedScale );

	const float Aspect = ( float ) Resolution.X / ( float ) Resolution.Y;

	RootComponent->SetWorldLocation( AnimatedTransform.GetLocation() );
	RootComponent->SetWorldRotation( AnimatedTransform.GetRotation() );
	
	// Update the window border mesh
	const float WindowMeshSize = 20.0f;	// Size of imported mesh, we need to inverse compensate for
	const FVector WindowMeshScale = FVector(
		GetSize().X / WindowMeshSize,
		GetSize().Y / WindowMeshSize,
		1.0f) * AnimatedScale * WorldScaleFactor;
	WindowMeshComponent->SetRelativeScale3D(WindowMeshScale);

	CentralWidgetComponent->SetRelativeScale3D(FVector(1.0f / 100.0f) * WorldScaleFactor);
	for (int32 Index = 0; Index < WidgetComponents.Num(); Index++)
	{
		const float Radius = 14.0f * WorldScaleFactor;
		float RadialX = -Radius * FMath::Sin(Index * ((2.0f*PI) / NumberOfEntries));
		float RadialY = Radius * FMath::Cos(Index * ((2.0f*PI) / NumberOfEntries));

		WidgetComponents[Index]->SetRelativeLocation(FVector(0.0f, RadialX, RadialY));
	}
	CentralWidgetComponent->SetRelativeLocation(FVector(2.0f*(WorldScaleFactor), 0.0f, 0.0f));
	WindowMeshComponent->SetRelativeLocation(FVector(-4.0f*(WorldScaleFactor), 0.0f, 0.0f));
}


void AVREditorRadialFloatingUI::UpdateFadingState( const float DeltaTime )
{
	if (WindowMeshComponent != nullptr)
	{
		UMaterialInstanceDynamic* DiskMaterial = Cast<UMaterialInstanceDynamic>(WindowMeshComponent->GetMaterial(0));
		if (GlowAmount > DefaultGlowAmount)
		{
			GlowAmount = FMath::Max(DefaultGlowAmount, GlowAmount - VREd::RadialUIFadeSpeed->GetFloat() * DeltaTime);
		}
		DiskMaterial->SetScalarParameterValue("GlowAmount", GlowAmount);
	}
	if (ArrowMeshComponent != nullptr)
	{
		UMaterialInstanceDynamic* ArrowMaterial = Cast<UMaterialInstanceDynamic>(ArrowMeshComponent->GetMaterial(0));
		if (ArrowAlpha < 1.0 && ArrowMeshComponent->IsVisible())
		{
			ArrowAlpha = FMath::Min(1.0f, ArrowAlpha + VREd::RadialUIFadeSpeed->GetFloat() * DeltaTime);
		}
		if (!ArrowMeshComponent->IsVisible())
		{
			ArrowAlpha = 0.0f;
		}
		ArrowMaterial->SetScalarParameterValue("Alpha", ArrowAlpha);
	}
	if (FadeDelay > 0.f)
	{
		FadeDelay -= DeltaTime;
	}
	else
	{
		if (bShouldBeVisible.GetValue())
		{
			FadeAlpha += VREd::RadialUIFadeSpeed->GetFloat() * DeltaTime;
		}
		else
		{
			FadeAlpha -= VREd::RadialUIFadeSpeed->GetFloat() * DeltaTime;
		}
		FadeAlpha = FMath::Clamp(FadeAlpha, 0.0f, 1.0f);

		if (FadeAlpha > 0.0f + KINDA_SMALL_NUMBER)
		{
			// At least a little bit visible
			if(IsHidden())
			{
				SetHidden(false);
				TInlineComponentArray<USceneComponent*> ComponentArray;
				GetComponents(ComponentArray);
				for (USceneComponent* Component : ComponentArray)
				{
					Component->SetVisibility(true);
				}
				FadeDelay = 0.0f;
			}
		}

		if (FadeAlpha >= 1.0f - KINDA_SMALL_NUMBER)
		{
			// Fully visible
		}
		else if (FadeAlpha <= 0.0f + KINDA_SMALL_NUMBER)
		{
			// Fully invisible
			if (!IsHidden())
			{
				SetHidden(true);
				TInlineComponentArray<USceneComponent*> ComponentArray;
				GetComponents(ComponentArray);
				for (USceneComponent* Component : ComponentArray)
				{
					Component->SetVisibility(false);
				}
				FadeDelay = 0.0f;
			}
		}

 		// Set material color
		const float UIBrightness = FadeAlpha * GetDefault<UVRModeSettings>()->UIBrightness;;
		for (UVREditorWidgetComponent* WidgetComponent : WidgetComponents)
		{
			WidgetComponent->SetTintColorAndOpacity(FLinearColor(UIBrightness, UIBrightness, UIBrightness).CopyWithNewOpacity(FadeAlpha));
		}

	}
}

FVector AVREditorRadialFloatingUI::CalculateAnimatedScale() const
{
	const float AnimationOvershootAmount = 0.7f;	// @todo vreditor tweak
	float EasedAlpha = UVREditorMode::OvershootEaseOut( FadeAlpha, AnimationOvershootAmount );
	EasedAlpha = FMath::Clamp(EasedAlpha, 0.01f, 1.0f + AnimationOvershootAmount);

	// Animate vertically more than horizontally; just looks a little better
	const float ZScale = FMath::Max( 0.001f, EasedAlpha );
	const float YScale = FMath::Max( 0.001f, 0.7f + 0.3f * EasedAlpha );

	FVector AnimatedScale = FVector( 1.0f, YScale, ZScale );
	AnimatedScale.Y *= YScale;
	AnimatedScale.Z *= ZScale;

	return AnimatedScale;
}

void AVREditorRadialFloatingUI::SetCollision(const ECollisionEnabled::Type InCollisionType, const ECollisionResponse InCollisionResponse, const ECollisionChannel InCollisionChannel)
{
	WindowMeshComponent->SetCollisionEnabled(InCollisionType);
	WindowMeshComponent->SetCollisionResponseToAllChannels(InCollisionResponse);
	WindowMeshComponent->SetCollisionObjectType(InCollisionChannel);

	for (UVREditorWidgetComponent* WidgetComponent : WidgetComponents)
	{
		WidgetComponent->SetCollisionEnabled(InCollisionType);
		WidgetComponent->SetCollisionResponseToAllChannels(InCollisionResponse);
		WidgetComponent->SetCollisionObjectType(InCollisionChannel);
	}
}

float AVREditorRadialFloatingUI::GetInitialScale() const
{
	return InitialScale;
}


void AVREditorRadialFloatingUI::ShowUI( const bool bShow, const bool bAllowFading, const float InitFadeDelay, const bool bPlaySound /*= true*/ )
{
	if( !bShouldBeVisible.IsSet() || bShow != bShouldBeVisible.GetValue() )
	{
		bShouldBeVisible = bShow;

		if( !bAllowFading )
		{
			SetHidden(!bShow);
			TInlineComponentArray<USceneComponent*> ComponentArray;
			GetComponents(ComponentArray);
			for (USceneComponent* Component : ComponentArray)
			{
				Component->SetVisibility(bShow);
			}
			FadeAlpha = bShow ? 1.0f : 0.0f;
		}

		// Set collision on components
		if (bShow)
		{
			SetCollision(ECollisionEnabled::QueryOnly, ECollisionResponse::ECR_Block, ECollisionChannel::ECC_WorldStatic);
		}
		else
		{
			SetCollision(ECollisionEnabled::NoCollision, ECollisionResponse::ECR_Ignore, ECollisionChannel::ECC_Visibility);
		}

		if (VRMode != nullptr && bPlaySound)
		{
			const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();
			VRMode->PlaySound(bShow ? AssetContainer.RadialMenuOpenSound : AssetContainer.RadialMenuCloseSound, GetActorLocation());
		}

		FadeDelay = InitFadeDelay;
	}
}


FVector2D AVREditorRadialFloatingUI::GetSize() const
{
	const float Aspect = (float)Resolution.X / (float)Resolution.Y;
	return FVector2D( Scale, Scale / Aspect );
}

float AVREditorRadialFloatingUI::GetScale() const
{
	return Scale;
}

void AVREditorRadialFloatingUI::SetScale( const float NewSize )
{
	Scale = NewSize;
	const float WorldScaleFactor = Owner->GetOwner().GetWorldScaleFactor();
	const FVector NewScale( Scale * WorldScaleFactor );
	const float Aspect = (float)Resolution.X / (float)Resolution.Y;
}


const void AVREditorRadialFloatingUI::HighlightSlot(const FVector2D& TrackpadPosition)
{
	if (TrackpadPosition.GetAbsMax() < VREd::MinJoystickOffsetBeforeRadialMenu->GetFloat())
	{
		if (CurrentlyHoveredButton.Get() != nullptr)
		{
			const FPointerEvent& SimulatedPointer = FPointerEvent();
			CurrentlyHoveredButton->OnMouseLeave(SimulatedPointer);
			if (CurrentlyHoveredWidget != nullptr)
			{
				Owner->OnHoverEndEffect(CurrentlyHoveredWidget);
			}
			CurrentlyHoveredButton = nullptr;
		}
		ArrowMeshComponent->SetVisibility(false);
		return;
	}

	const float AnglePerItem = 360.0f / NumberOfEntries;
	float Angle = (float) FRotator::NormalizeAxis(FMath::RadiansToDegrees(FMath::Atan2(TrackpadPosition.X, TrackpadPosition.Y)));

	// first element of the menu is at 90
	float ArrowAngle = Angle - 90.0f;
	if (ArrowAngle < 0)
	{
		ArrowAngle = ArrowAngle + 360.0f;
	}
	if (ArrowAngle > 360.0f)
	{
		ArrowAngle = ArrowAngle - 360.0f;
	}

	Angle += AnglePerItem / 2.0f;
	
	if (Angle < 0)
	{
		Angle = Angle + 360.0f;
	}
	if (Angle > 360.0f)
	{
		Angle = Angle - 360.0f;
	}
	ArrowMeshComponent->SetRelativeRotation(FRotator(0.0f, ArrowAngle, 0.0f).Quaternion());
	float NewArrowScaleFactor = (float) TrackpadPosition.Size();
	if (TrackpadPosition.GetAbsMax() > VREd::MinJoystickOffsetBeforeRadialMenu->GetFloat())
	{
		ArrowMeshComponent->SetVisibility(true);
		if (NewArrowScaleFactor > 0.8f)
		{
			NewArrowScaleFactor = 1.0f;
		}
		const FVector NewArrowScale = FVector(NewArrowScaleFactor, NewArrowScaleFactor, 2.0f * NewArrowScaleFactor);
		ArrowMeshComponent->SetRelativeScale3D(NewArrowScale);
	}
	else
	{
		ArrowMeshComponent->SetVisibility(false);
	}

	const FPointerEvent& SimulatedPointer = FPointerEvent();
	const FGeometry& ChildGeometry = FGeometry();
	TSharedRef<SWidget> TestWidget = SNullWidget::NullWidget;
	const int32 Index = (int32) (Angle / AnglePerItem);
	// Make sure we are checking for a valid radial menu widget
	if (WidgetComponents.IsValidIndex(Index) && WidgetComponents[Index]->GetSlateWidget())
	{
		TSharedRef<SWidget> CurrentChild = WidgetComponents[Index]->GetSlateWidget().ToSharedRef();
		TestWidget = UVREditorUISystem::FindWidgetOfType(CurrentChild, ButtonTypeOverride);
	}
	if (TestWidget != SNullWidget::NullWidget)
	{
		CurrentlyHoveredButton = StaticCastSharedRef<SButton>(TestWidget);
		CurrentlyHoveredWidget = WidgetComponents[Index];
	
		// Simulate mouse entering event for the button if it was not previously hovered
		if (!(CurrentlyHoveredButton->IsHovered()))
		{
			CurrentlyHoveredButton->OnMouseEnter(ChildGeometry, SimulatedPointer);
			Owner->OnHoverBeginEffect(CurrentlyHoveredWidget);
		}
	}
	// Simulate mouse leaving events for any buttons that were previously hovered
	for (int32 ButtonCount = 0; ButtonCount < (NumberOfEntries); ButtonCount++)
	{
		if (ButtonCount != Index && WidgetComponents.IsValidIndex(ButtonCount) && WidgetComponents[ButtonCount]->GetSlateWidget())
		{
			TSharedRef<SWidget> ChildWidget = WidgetComponents[ButtonCount]->GetSlateWidget().ToSharedRef();
			TestWidget = UVREditorUISystem::FindWidgetOfType(ChildWidget, ButtonTypeOverride);
			TSharedRef<SButton> TestButton = StaticCastSharedRef<SButton>(TestWidget);
			if (TestButton->IsHovered())
			{
				TestButton->OnMouseLeave(SimulatedPointer);
				Owner->OnHoverEndEffect(WidgetComponents[ButtonCount]);
			}
		}

	}
}

void AVREditorRadialFloatingUI::SimulateLeftClick()
{
	// TODO: Change this to one supported input button
	if (CurrentlyHoveredButton.Get() != nullptr)
	{
		if (ButtonTypeOverride == FName(TEXT("SMenuEntryButton")))
		{
			FSlateApplication::Get().SetKeyboardFocus(CurrentlyHoveredButton, EFocusCause::SetDirectly);
			FVREditorActionCallbacks::SimulateKeyDown(EKeys::Enter, false);
			FVREditorActionCallbacks::SimulateKeyUp(EKeys::Enter);
		}
		if (ButtonTypeOverride == FName(TEXT("SButton")))
		{
			const FPointerEvent& SimulatedPointer = FPointerEvent(uint32(0), uint32(0), FVector2D::ZeroVector, FVector2D::ZeroVector, TSet<FKey>(), EKeys::LeftMouseButton, 0.0f, FModifierKeysState());
			const FGeometry& ChildGeometry = FGeometry();
			CurrentlyHoveredButton->OnMouseButtonDown(ChildGeometry, SimulatedPointer);
			CurrentlyHoveredButton->OnMouseButtonUp(ChildGeometry, SimulatedPointer);
		}
		UMaterialInstanceDynamic* DiskMaterial = Cast<UMaterialInstanceDynamic>(WindowMeshComponent->GetMaterial(0));
		GlowAmount = 2.0f*DefaultGlowAmount;
		DiskMaterial->SetScalarParameterValue("GlowAmount", GlowAmount);
	}
}

const TSharedPtr<SButton>& AVREditorRadialFloatingUI::GetCurrentlyHoveredButton()
{
	return CurrentlyHoveredButton;
}

void AVREditorRadialFloatingUI::UpdateCentralWidgetComponent(const TSharedPtr<SWidget>& NewCentralSlateWidget)
{
	// NOTE: Must be called *after* RegisterComponent() because UWidgetComponent nulls out Widget if no WidgetClass is set (WidgetClass is protected and there is no accessor)
	if (NewCentralSlateWidget.IsValid())
	{
		CentralSlateWidget = NewCentralSlateWidget;
		if (CentralWidgetComponent != nullptr)
		{
			CentralWidgetComponent->SetSlateWidget(NewCentralSlateWidget.ToSharedRef());
		}
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
