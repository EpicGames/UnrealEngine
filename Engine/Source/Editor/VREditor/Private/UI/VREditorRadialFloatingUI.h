// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "VREditorBaseActor.h"
#include "Widgets/Input/SButton.h"
#include "VREditorRadialFloatingUI.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UVREditorBaseUserWidget;
class UVREditorUISystem;

/**
 * Represents an interactive floating UI panel in the VR Editor
 */
UCLASS()
class AVREditorRadialFloatingUI : public AVREditorBaseActor
{
	GENERATED_BODY()

public:

	/** Default constructor which sets up safe defaults */
	AVREditorRadialFloatingUI();
	virtual void PostActorCreated() override;

	/** Creates a FVREditorFloatingUI using a Slate widget, and sets up safe defaults */
	void SetSlateWidget( class UVREditorUISystem& InitOwner, const TSharedRef<SWidget>& InitSlateWidget, const FIntPoint InitResolution, const float InitScale, const EDockedTo InitDockedTo );

	/** @return Returns true if the UI is visible (or wants to be visible -- it might be transitioning */
	bool IsUIVisible() const
	{
		return bShouldBeVisible.GetValue();
	}

	/** Shows or hides the UI (also enables collision, and performs a transition effect) */
	void ShowUI( const bool bShow, const bool bAllowFading = true, const float InitFadeDelay = 0.0f, const bool bPlaySound = true );

	/** Returns the widget component for this UI, or nullptr if not spawned right now */
	const TArray<class UVREditorWidgetComponent*>& GetWidgetComponents() const
	{
		return WidgetComponents;
	}

	/** Returns the mesh component for this UI, or nullptr if not spawned right now */
	class UStaticMeshComponent* GetMeshComponent()
	{
		return WindowMeshComponent;
	}

	/** Returns the actual size of the UI in either axis, after scaling has been applied.  Does not take into animation or world scaling */
	FVector2D GetSize() const;

	/** Gets the scale */
	float GetScale() const;

	/** Sets a new size for the UI */
	void SetScale( const float NewSize );

	/** Sets the UI transform */
	virtual void SetTransform( const FTransform& Transform ) override;

	/** AActor overrides */
	virtual void Destroyed() override;
	virtual bool IsEditorOnly() const final
	{
		return true;
	}
	//~ End AActor interface

	/** Returns the owner of this object */
	UVREditorUISystem& GetOwner()
	{
		check( Owner != nullptr );
		return *Owner;
	}

	/** Returns the owner of this object (const) */
	const UVREditorUISystem& GetOwner() const
	{
		check( Owner != nullptr );
		return *Owner;
	}

	/** Gets the initial size of this UI */
	float GetInitialScale() const;

	/** Called to finish setting everything up, after a widget has been assigned */
	virtual void SetupWidgetComponent(TSharedPtr<SWidget> SlateWidget);

	/**Reset for the next radial menu to be created */
	void Reset();

	/** Set the number of entries for the menu */
	void SetNumberOfEntries(const float InNumberOfEntries)
	{
		NumberOfEntries = InNumberOfEntries;
	};

	/** Set the button type for the menu */
	void SetButtonTypeOverride(const FName InButtonTypeOverride)
	{
		ButtonTypeOverride = InButtonTypeOverride;
	}

	/** Store the menu widget for the menu */
	void SetCurrentMenuWidget(const TSharedPtr<class SMultiBoxWidget> InWidget)
	{
		MenuMultiBoxWidget = InWidget;
	}

	/** Return the menu widget for comparison*/
	const TSharedPtr<class SMultiBoxWidget> GetCurrentMenuWidget()
	{
		return MenuMultiBoxWidget;
	}

	/**
	* Highlight the widget in a slot based on a given trackpad position
	*
	* @param	TrackpadPosition	The current trackpad or thumbstick position
	*/
	const void HighlightSlot(const FVector2D& TrackpadPosition);

	/** Simulate a left-mouse click (down and up) on the currently hovered button */
	void SimulateLeftClick();

	/** Gets the currently hovered button */
	const TSharedPtr<SButton>& GetCurrentlyHoveredButton();

	void UpdateCentralWidgetComponent(const TSharedPtr<SWidget>& NewCentralSlateWidget);

protected:

	/** Returns a scale to use for this widget that takes into account animation */
	FVector CalculateAnimatedScale() const;

	/** Set collision on components */
	virtual void SetCollision(const ECollisionEnabled::Type InCollisionType, const ECollisionResponse InCollisionResponse, const ECollisionChannel InCollisionChannel);

private:

	/** Called after spawning, and every Tick, to update opacity of the widget */
	virtual void UpdateFadingState( const float DeltaTime );

protected:
	/** Stores the widget associated with the quick menu*/
	TSharedPtr<class SMultiBoxWidget> MenuMultiBoxWidget;

	/** Slate widget we're drawing, or null if we're drawing a UMG user widget */
	TArray<TSharedPtr<SWidget>> SlateWidgets;
	
	/** When in a spawned state, this is the widget component to represent the widget */
	UPROPERTY()
	TArray<TObjectPtr<class UVREditorWidgetComponent>> WidgetComponents;

	/** The floating window mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> WindowMeshComponent;

	/** The arrow indicator mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> ArrowMeshComponent;

	/** The central helper widget */
	UPROPERTY()
	TObjectPtr<class UVREditorWidgetComponent> CentralWidgetComponent;

	TSharedPtr<SWidget> CentralSlateWidget;

	/** Resolution we should draw this UI at, regardless of scale */
	FIntPoint Resolution;

private:

	/** Owning object */
	class UVREditorUISystem* Owner;

	/** True if the UI wants to be visible, or false if it wants to be hidden.  Remember, we might still be visually 
	    transitioning between states */
	TOptional<bool> bShouldBeVisible;

	/** Fade alpha, for visibility transitions */
	float FadeAlpha;

	/** Delay to fading in or out. Set on ShowUI and cleared on finish of fade in/out */
	float FadeDelay;

	/** The starting scale of this UI */
	float InitialScale;

	/** Number of entries in the radial menu */
	float NumberOfEntries;

	/** Radial menu supports SButton and SMenuEntryButton, but needs to know which type it has */
	FName ButtonTypeOverride;

	/** Stores the currently hovered button */
	TSharedPtr<SButton> CurrentlyHoveredButton;

	/** Stores the currently hovered widget component */
	class UVREditorWidgetComponent* CurrentlyHoveredWidget;

	/** Glow amount of window frame - VFX */
	float GlowAmount;

	float DefaultGlowAmount;

	/** Alpha of arrow - VFX */
	float ArrowAlpha;

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
