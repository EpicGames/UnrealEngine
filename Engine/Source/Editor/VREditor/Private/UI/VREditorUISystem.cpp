// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VREditorUISystem.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"
#include "GenericPlatform/GenericApplication.h"
#include "Modules/ModuleManager.h"
#include "InputCoreTypes.h"
#include "Input/Events.h"
#include "Input/Reply.h"

#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Layout/WidgetPath.h"

#include "Engine/EngineTypes.h"
#include "Editor/UnrealEdEngine.h"
#include "Sound/SoundCue.h"
#include "UnrealEdGlobals.h"

#include "VREditorMode.h"
#include "VREditorBaseActor.h"
#include "UI/VREditorBaseUserWidget.h"
#include "UI/VREditorFloatingUI.h"
#include "UI/VREditorRadialFloatingUI.h"
#include "UI/VREditorDockableWindow.h"
#include "UI/VREditorDockableCameraWindow.h"
#include "ViewportInteractionTypes.h"
#include "IHeadMountedDisplay.h"
#include "ViewportWorldInteraction.h"
#include "VREditorInteractor.h"
#include "IVREditorModule.h"
#include "VREditorAssetContainer.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Kismet2/DebuggerCommands.h"

// UI
#include "Components/WidgetComponent.h"
#include "Components/StaticMeshComponent.h"
#include "VREditorWidgetComponent.h"
#include "VREditorStyle.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"


// Actor Details, Modes
#include "SEditorViewport.h"

#include "ILevelEditor.h"
#include "LevelEditor.h"

// World Settings

#include "SLevelViewport.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/SWidget.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/MultiBox/MultiBox.h"
#include "LevelEditorActions.h"
#include "VREditorActions.h"
#include "Framework/Commands/GenericCommands.h"
#include "Components/StaticMeshComponent.h"

#include "ISequencer.h"
#include "ITimeSlider.h"
#include "Engine/StaticMesh.h"
#include "EdMode.h"
#include "Widgets/Images/SImage.h"

#include "DrawDebugHelpers.h"
#include "VRModeSettings.h"
#include "SequencerSettings.h"
#include "Engine/Selection.h"
#include "UI/VREditorFloatingCameraUI.h"
#include "ScopedTransaction.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "EditorModes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorUISystem)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "VREditor"

namespace VREd
{
	static FAutoConsoleVariable SequencerUIResolutionX(TEXT("VREd.SequencerUIResolutionX"), 960, TEXT("Horizontal resolution to use for Sequencer UI render targets"));
	static FAutoConsoleVariable SequencerUIResolutionY(TEXT("VREd.SequencerUIResolutionY"), 600, TEXT("Vertical resolution to use for Sequencer UI render targets"));
	static FAutoConsoleVariable DefaultEditorUIResolutionX( TEXT( "VREd.DefaultEditorUIResolutionX" ), 1024, TEXT( "Horizontal resolution to use for VR editor UI render targets" ) );
	static FAutoConsoleVariable DefaultEditorUIResolutionY( TEXT( "VREd.DefaultEditorUIResolutionY" ), 1024, TEXT( "Vertical resolution to use for VR editor UI render targets" ) );
	static FAutoConsoleVariable DefaultRadialElementResolutionX(TEXT("VREd.DefaultRadialElementResolutionX"), 350, TEXT("Horizontal resolution to use for VR editor radial UI render targets"));
	static FAutoConsoleVariable DefaultRadialElementResolutionY(TEXT("VREd.DefaultRadialElementResolutionY"), 350, TEXT("Vertical resolution to use for VR editor radial UI render targets"));
	static FAutoConsoleVariable QuickMenuUIResolutionX( TEXT( "VREd.QuickMenuUIResolutionX" ), 1200, TEXT( "Horizontal resolution to use for Quick Menu VR UI render targets" ) );
	static FAutoConsoleVariable QuickMenuUIResolutionY( TEXT( "VREd.QuickMenuUIResolutionY" ), 1075, TEXT( "Vertical resolution to use for Quick Menu VR UI render targets" ) );
	static FAutoConsoleVariable EditorUISize( TEXT( "VREd.EditorUISize" ), 70.0f, TEXT( "How big editor UIs should be" ) );
	static FAutoConsoleVariable EditorUIScale( TEXT( "VREd.EditorUIScale" ), 2.0f, TEXT( "How much to scale up (or down) editor UIs for VR" ) );
	static FAutoConsoleVariable AssetEditorUIResolutionX( TEXT( "VREd.AssetEditorUIResolutionX" ), 1920, TEXT( "Horizontal resolution to use for VR editor asset editor UI render targets" ) );
	static FAutoConsoleVariable AssetEditorUIResolutionY( TEXT( "VREd.AssetEditorUIResolutionY" ), 1080, TEXT( "Vertical resolution to use for VR editor asset editor UI render targets" ) );
	static FAutoConsoleVariable RadialMenuFadeDelay( TEXT( "VREd.RadialMenuFadeDelay" ), 0.2f, TEXT( "The delay for the radial menu after selecting a button" ) );
	static FAutoConsoleVariable UIAbsoluteScrollSpeed( TEXT( "VREd.UIAbsoluteScrollSpeed" ), 8.0f, TEXT( "How fast the UI scrolls when dragging the touchpad" ) );
	static FAutoConsoleVariable UIRelativeScrollSpeed( TEXT( "VREd.UIRelativeScrollSpeed" ), 0.75f, TEXT( "How fast the UI scrolls when holding an analog stick" ) );
	static FAutoConsoleVariable MinUIScrollDeltaForInertia( TEXT( "VREd.MinUIScrollDeltaForInertia" ), 0.25f, TEXT( "Minimum amount of touch pad input before inertial UI scrolling kicks in" ) );
	static FAutoConsoleVariable UIPressHapticFeedbackStrength( TEXT( "VREd.UIPressHapticFeedbackStrength" ), 0.4f, TEXT( "Strenth of haptic when clicking on the UI" ) );
	static FAutoConsoleVariable UIAssetEditorSummonedOnHandHapticFeedbackStrength( TEXT( "VREd.UIAssetEditorSummonedOnHandHapticFeedbackStrength" ), 1.0f, TEXT( "Strenth of haptic to play to remind a user which hand an asset editor was spawned on" ) );
	static FAutoConsoleVariable MaxDockWindowSize( TEXT( "VREd.MaxDockWindowSize" ), 250, TEXT( "Maximum size for dockable windows" ) );
	static FAutoConsoleVariable MinDockWindowSize( TEXT( "VREd.MinDockWindowSize" ), 40, TEXT( "Minimum size for dockable windows" ) );
	static FAutoConsoleVariable UIPanelOpenDistance( TEXT( "VREd.UIPanelOpenDistance" ), 20.f, TEXT( "Distance to spawn a panel from the hand in centimeters" ) );
	static FAutoConsoleVariable UIPanelOpenRotationPitchOffset( TEXT( "VREd.UIPanelOpenRotationPitchOffset" ), 45.0f, TEXT( "The pitch rotation offset in degrees when spawning a panel in front of the motioncontroller" ) );
	static FAutoConsoleVariable SteamVRTrackpadDeadzone(TEXT("VREd.SteamVRTrackpadDeadzone"), 0.3f, TEXT("The deadzone for the Vive motion controller trackpad"));
	static const FTransform DefaultColorPickerTransform(	FRotator( -10, 180, 0), FVector( 30, 35, 0 ), FVector( 1 ) );
	static const FTransform DefaultActorPreviewTransform(FRotator(-20, 120, 0), FVector(50, -60, 0), FVector(1));
	static FAutoConsoleVariable DefaultCameraUIResolutionX( TEXT( "VREd.DefaultCameraUIResolutionX" ), 1400, TEXT( "Horizontal resolution to use for VR editor UI render targets" ) );
	static FAutoConsoleVariable DefaultCameraUIResolutionY( TEXT( "VREd.DefaultCameraUIResolutionY" ), 787, TEXT( "Vertical resolution to use for VR editor UI render targets" ) );
	static FAutoConsoleVariable CameraPreviewUISize( TEXT( "VREd.CameraPreviewUISize" ), 50.0f, TEXT( "How big camera preview UIs should be" ) );
}


const VREditorPanelID UVREditorUISystem::SequencerPanelID = VREditorPanelID("SequencerUI");
const VREditorPanelID UVREditorUISystem::InfoDisplayPanelID = VREditorPanelID("InfoDisplay");
const VREditorPanelID UVREditorUISystem::RadialMenuPanelID = VREditorPanelID("RadialMenu");
const VREditorPanelID UVREditorUISystem::TabManagerPanelID = VREditorPanelID("TabManagerPanel");
const FString ActorPreviewPrefix = TEXT("ActorPreview");

UVREditorUISystem::UVREditorUISystem() : 
	Super(),
	VRMode( nullptr ),
	FloatingUIs(),
	PreviewWindowInfo(),
	InfoDisplayPanel( nullptr ),
	QuickRadialMenu( nullptr ),
	RadialMenuHideDelayTime( 0.0f ),
	InteractorDraggingUI( nullptr ),
	DraggingUIOffsetTransform( FTransform::Identity ),
	DraggingUI( nullptr ),
	ColorPickerUI( nullptr ),
	bRefocusViewport(false),
	LastDraggingHoverLocation( FVector::ZeroVector ),
	bSequencerOpenedFromRadialMenu(false),
	bDragPanelFromOpen(false),
	DragPanelFromOpenTime(0.0f),
	bPanelCanScale(true)
{
}

void UVREditorUISystem::Init(UVREditorMode* InVRMode)
{
	check (InVRMode != nullptr);
	VRMode = InVRMode;

	// Register to find out about VR events
	GetOwner().GetWorldInteraction().OnPreviewInputAction().AddUObject( this, &UVREditorUISystem::OnPreviewInputAction );
	GetOwner().GetWorldInteraction().OnViewportInteractionInputAction().AddUObject( this, &UVREditorUISystem::OnVRAction );
	GetOwner().GetWorldInteraction().OnViewportInteractionHoverUpdate().AddUObject( this, &UVREditorUISystem::OnVRHoverUpdate );
	GetOwner().OnToggleDebugMode().AddUObject(this, &UVREditorUISystem::ToggledDebugMode);

	FSlateApplication::Get().OnDragDropCheckOverride.BindUObject(this, &UVREditorUISystem::CheckForVRDragDrop);

	UpdateInteractors();

	// Create all of our UI panels
	bRadialMenuIsNumpad = false;
	RadialMenuHandler = NewObject<UVRRadialMenuHandler>();
	RadialMenuHandler->Init(this);

	CreateUIs();

	// Bind the global tab manager's dockable area restore override
	FGlobalTabmanager::Get()->OnOverrideDockableAreaRestore_Handler.BindUObject(this, &UVREditorUISystem::DockableAreaRestored);

	FVREditorActionCallbacks::GizmoCoordinateSystemText = FVREditorActionCallbacks::GetGizmoCoordinateSystemText();
	FVREditorActionCallbacks::GizmoModeText = FVREditorActionCallbacks::GetGizmoModeText();
	FVREditorActionCallbacks::UpdateSelectingCandidateActorsText(VRMode);
	FVREditorActionCallbacks::SelectingCandidateActorsText = FVREditorActionCallbacks::GetSelectingCandidateActorsText();

	AssetEditorCloseDelegate = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().AddUObject(this, &UVREditorUISystem::OnCloseAssetEditor);

}

void UVREditorUISystem::UpdateInteractors( )
{
	// TODO: asymmetry hardcoded right now
	if (GetDefault<UVRModeSettings>()->InteractorHand == EInteractorHand::Left)
	{
		LaserInteractor = Cast<UVREditorInteractor>( GetOwner().GetHandInteractor( EControllerHand::Left ) );
		UIInteractor = Cast<UVREditorInteractor>( GetOwner().GetHandInteractor( EControllerHand::Right ) );
	}
	else
	{
		UIInteractor = Cast<UVREditorInteractor>( GetOwner().GetHandInteractor( EControllerHand::Left ) );
		LaserInteractor = Cast<UVREditorInteractor>( GetOwner().GetHandInteractor( EControllerHand::Right ) );
	}
	UIInteractor->SetControllerType( EControllerType::UI );
	LaserInteractor->SetControllerType(EControllerType::Laser);
}


void UVREditorUISystem::Shutdown()
{
	FSlateApplication::Get().OnDragDropCheckOverride.Unbind();

	if (VRMode != nullptr)
	{
		UViewportWorldInteraction& WorldInteraction = VRMode->GetWorldInteraction();
		WorldInteraction.OnPreviewInputAction().RemoveAll(this);
		WorldInteraction.OnViewportInteractionInputAction().RemoveAll(this);
		WorldInteraction.OnViewportInteractionHoverUpdate().RemoveAll(this);
		VRMode->OnToggleDebugMode().RemoveAll(this);
	}

	GLevelEditorModeTools().OnEditorModeIDChanged().RemoveAll(this);

	// Unbind the color picker creation & destruction overrides
	SColorPicker::OnColorPickerNonModalCreateOverride.Unbind();
	SColorPicker::OnColorPickerDestroyOverride.Unbind();
	FGlobalTabmanager::Get()->OnOverrideDockableAreaRestore_Handler.Unbind();

	// If we have a sequence tab open, reset its widget and close the associated Sequencer
	if (GetOwner().GetCurrentSequencer() != nullptr)
	{
		AVREditorFloatingUI* SequencerPanel = GetPanel(SequencerPanelID);
		if (SequencerPanel != nullptr)
		{
			SequencerPanel->SetSlateWidget(*this, SequencerPanelID, SNullWidget::NullWidget, FIntPoint((int32)VREd::SequencerUIResolutionX->GetFloat(), (int32)VREd::SequencerUIResolutionY->GetFloat()), VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
		}
		FVREditorActionCallbacks::CloseSequencer(GetOwner().GetCurrentSequencer()->GetRootMovieSceneSequence());
	}

	if (InfoDisplayPanel != nullptr)
	{
		InfoDisplayPanel->SetSlateWidget(SNullWidget::NullWidget);
	}

	for (auto& CurrentUI : FloatingUIs)
	{
		AVREditorFloatingUI* UIPanel = CurrentUI.Value;
		if (UIPanel != nullptr)
		{
			VRMode->DestroyTransientActor(UIPanel);
		}
	}

	FloatingUIs.Reset();
	PreviewWindowInfo.Reset();
	VRMode->DestroyTransientActor(QuickRadialMenu);
	QuickRadialMenu = nullptr;
	InfoDisplayPanel = nullptr;
	CurrentWidgetOnInfoDisplay.Reset();

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	LevelEditorModule.GetLevelEditorTabManager().ToSharedRef()->SetCanDoDragOperation(true);

	ProxyTabManager.Reset();

	// Remove the proxy tab manager, we don't want to steal tabs any more.
	FGlobalTabmanager::Get()->SetProxyTabManager(TSharedPtr<FProxyTabmanager>());
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorOpened().RemoveAll(this);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().RemoveAll(this);
	VRMode = nullptr;
	DraggingUI = nullptr;
	ColorPickerUI = nullptr;
}


void UVREditorUISystem::OnPreviewInputAction(FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor,
	const FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled)
{
	static const FName SteamVR(TEXT("SteamVR"));
	static const FName OculusHMD(TEXT("OculusHMD"));

	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>(Interactor);
	
	// If we are releasing a UI panel that started drag from opening it.
	if (VREditorInteractor && UIInteractor != nullptr && DraggingUI != nullptr && InteractorDraggingUI != nullptr && UIInteractor == InteractorDraggingUI && VREditorInteractor == UIInteractor && 
		 Action.Event == EInputEvent::IE_Released && 
		((VRMode->GetHMDDeviceType() != SteamVR && Action.ActionType == ViewportWorldActionTypes::SelectAndMove) || (VRMode->GetHMDDeviceType() == SteamVR && Action.ActionType == VRActionTypes::ConfirmRadialSelection)))
	{
		bDragPanelFromOpen = false;
		UViewportDragOperationComponent* DragOperationComponent = DraggingUI->GetDragOperationComponent();
		if (DragOperationComponent)
		{
			DragOperationComponent->ClearDragOperation();
		}
		const bool bPlaySound = false;
		StopDraggingDockUI(InteractorDraggingUI, bPlaySound);
	}
	
	// UI Interactor Preview actions
	if (VREditorInteractor &&
		VREditorInteractor == UIInteractor && 
		(VREditorInteractor->GetDraggingMode() != EViewportInteractionDraggingMode::World || (VREditorInteractor->GetOtherInteractor() != nullptr && VREditorInteractor->GetOtherInteractor()->GetDraggingMode() != EViewportInteractionDraggingMode::World && VREditorInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::AssistingDrag )))
	{
		if (Action.Event == EInputEvent::IE_Pressed && 
			VRMode->GetHMDDeviceType() == OculusHMD 
			&& Action.ActionType == VRActionTypes::ConfirmRadialSelection
			&& GIsDemoMode)
		{
			ResetAll();
			bWasHandled = true;
		}

		if (!bWasHandled)
		{
			if (IsShowingRadialMenu(VREditorInteractor))
			{
				// If the numpad is currently showing and we press a button (only on press to avoid duplicate calls)
				if (bRadialMenuIsNumpad && Action.Event == IE_Pressed)
				{
					// Modifier button is backspace
					if (Action.ActionType == VRActionTypes::Modifier)
					{
						const bool bRepeat = false;
						FVREditorActionCallbacks::SimulateBackspace();
						bWasHandled = true;
					}
					// Side triggers function as enter keys
					if (Action.ActionType == ViewportWorldActionTypes::WorldMovement)
					{
						const bool bRepeat = false;
						FVREditorActionCallbacks::SimulateKeyDown(EKeys::Enter, bRepeat);
						FVREditorActionCallbacks::SimulateKeyUp(EKeys::Enter);
						// After pressing enter, dismiss the numpad
						SwapRadialMenu();
						if (!bRadialMenuVisibleAtSwap)
						{
							HideRadialMenu();
						}
						bWasHandled = true;
					}
				}
				if (Action.ActionType == VRActionTypes::Modifier && 
					Action.Event == IE_Pressed &&
					!bRadialMenuIsNumpad &&
					!bWasHandled)
				{
					if (RadialMenuHandler && RadialMenuHandler->GetCurrentMenuGenerator().GetHandle() != RadialMenuHandler->GetHomeMenuGenerator().GetHandle())
					{
						RadialMenuHandler->BackOutMenu();
					}
				}
				if (!bWasHandled)
				{
					if ((VRMode->GetHMDDeviceType() == OculusHMD && Action.ActionType == ViewportWorldActionTypes::SelectAndMove) || 
						(VRMode->GetHMDDeviceType() == SteamVR && Action.ActionType == VRActionTypes::ConfirmRadialSelection))
					{
						// If the radial menu is showing, select the currently highlighted button by pressing the trigger
						if (QuickRadialMenu->GetCurrentlyHoveredButton().IsValid())
						{
							if ((Action.Event == IE_Pressed))
							{
								QuickRadialMenu->SimulateLeftClick();
								bOutIsInputCaptured = true;
							}
							if (Action.Event == IE_Released)
							{
								bOutIsInputCaptured = false;
							}
							bWasHandled = true;
						}
					}
				}
			}	
			else if (Action.ActionType == VRActionTypes::TrackpadUp &&
				Action.Event == EInputEvent::IE_Pressed && 
				VREditorInteractor->GetDraggingMode() != EViewportInteractionDraggingMode::AssistingDrag &&
				VREditorInteractor->GetDraggingMode() != EViewportInteractionDraggingMode::TransformablesFreely)
			{
				const bool bForceRefresh = false;
				TryToSpawnRadialMenu(VREditorInteractor, bForceRefresh);
				bWasHandled = true;
			}
		}
	}

	// Laser Interaction Preview actions
	if (VREditorInteractor &&
		VREditorInteractor->GetControllerType() == EControllerType::Laser &&
		Action.ActionType == ViewportWorldActionTypes::SelectAndMove &&
		VREditorInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::Nothing)
	{
		FVector LaserPointerStart, LaserPointerEnd;
		// If we are clicking on an Actor but not a widget component, send a fake mouse click event to toggle focus
		if (VREditorInteractor->GetLaserPointer(LaserPointerStart, LaserPointerEnd))
		{
			FHitResult HitResult = VREditorInteractor->GetHitResultFromLaserPointer();
			if (HitResult.HitObjectHandle.IsValid())
			{
				UWidgetComponent* WidgetComponent = Cast<UWidgetComponent>(HitResult.GetComponent());

				if (WidgetComponent == nullptr)
				{
					// If we didn't handle the input in any other way, send an empty mouse down event so Slate focus is handled correctly
					const bool bIsRightClicking =
						(Action.Event == IE_Pressed && VREditorInteractor->IsModifierPressed()) ||
						(Action.Event == IE_Released && VREditorInteractor->IsRightClickingOnUI());
					TSet<FKey> PressedButtons;
					FPointerEvent PointerEvent(
						1 + (uint8)VREditorInteractor->GetControllerSide(),
						FVector2D::ZeroVector,
						FVector2D::ZeroVector,
						PressedButtons,
						bIsRightClicking ? EKeys::RightMouseButton : EKeys::LeftMouseButton,
						0.0f,	// Wheel delta
						FModifierKeysState());

					FWidgetPath EmptyWidgetPath;
					FReply Reply = FSlateApplication::Get().RoutePointerDownEvent(EmptyWidgetPath, PointerEvent);
					if (bRadialMenuIsNumpad)
					{
						// If clicking somewhere outside UI so the widget loses focus
						SwapRadialMenu();
						if (!bRadialMenuVisibleAtSwap)
						{
							HideRadialMenu();
						}
					}
				}
				else
				{
					// Only allow clicks to our own widget components
					// Always mark the event as handled so that the editor doesn't try to select the widget component
					bWasHandled = true;

					if (Action.Event != IE_Repeat)
					{
						// If the Modifier button is held down, treat this like a right click instead of a left click
						const bool bIsRightClicking =
							(Action.Event == IE_Pressed && VREditorInteractor->IsModifierPressed()) ||
							(Action.Event == IE_Released && VREditorInteractor->IsRightClickingOnUI());

						FVector2D LastLocalHitLocation = WidgetComponent->GetLastLocalHitLocation();

						FVector2D LocalHitLocation;
						WidgetComponent->GetLocalHitLocation(HitResult.ImpactPoint, LocalHitLocation);

						// If we weren't already hovering over this widget, then we'll reset the last hit location
						if (WidgetComponent != VREditorInteractor->GetLastHoveredWidgetComponent())
						{
							LastLocalHitLocation = LocalHitLocation;

							if (UVREditorWidgetComponent* VRWidgetComponent = Cast<UVREditorWidgetComponent>(VREditorInteractor->GetLastHoveredWidgetComponent()))
							{
								VRWidgetComponent->SetIsHovering(false);
								OnHoverEndEffect(VRWidgetComponent);
							}
						}

						FWidgetPath WidgetPathUnderFinger = FWidgetPath(WidgetComponent->GetHitWidgetPath(HitResult.ImpactPoint, /*bIgnoreEnabledStatus*/ false));
						if (WidgetPathUnderFinger.IsValid())
						{
							TSet<FKey> PressedButtons;
							if (Action.Event == IE_Pressed)
							{
								PressedButtons.Add(bIsRightClicking ? EKeys::RightMouseButton : EKeys::LeftMouseButton);
							}

							FPointerEvent PointerEvent(
								1 + (uint8)VREditorInteractor->GetControllerSide(),
								LocalHitLocation,
								LastLocalHitLocation,
								PressedButtons,
								bIsRightClicking ? EKeys::RightMouseButton : EKeys::LeftMouseButton,
								0.0f,	// Wheel delta
								FModifierKeysState());

							VREditorInteractor->SetLastHoveredWidgetComponent(WidgetComponent);

							if (UVREditorWidgetComponent* VRWidgetComponent = Cast<UVREditorWidgetComponent>(VREditorInteractor->GetLastHoveredWidgetComponent()))
							{
								VRWidgetComponent->SetIsHovering(true);
								OnHoverBeginEffect(VRWidgetComponent);
							}

							FReply Reply = FReply::Unhandled();
							if (Action.Event == IE_Pressed)
							{
								const double CurrentTime = FPlatformTime::Seconds();
								if (CurrentTime - VREditorInteractor->GetLastUIPressTime() <= GetDefault<UVRModeSettings>()->DoubleClickTime)
								{
									// Trigger a double click event!
									Reply = FSlateApplication::Get().RoutePointerDoubleClickEvent(WidgetPathUnderFinger, PointerEvent);
								}
								else
								{
									// If we are clicking on an editable text field and the radial menu is not a numpad, show the numpad
									if (WidgetPathUnderFinger.Widgets.Last().Widget->GetTypeAsString() == TEXT("SEditableText") && (bRadialMenuIsNumpad == false))
									{
										if (!QuickRadialMenu->IsHidden())
										{
											bRadialMenuVisibleAtSwap = true;
										}
										else
										{
											bRadialMenuVisibleAtSwap = false;
											// Force the radial menu to spawn even if the laser is over UI
											const bool bForceRefresh = false;
											TryToSpawnRadialMenu(UIInteractor, bForceRefresh);
										}
										SwapRadialMenu();
									}
									Reply = FSlateApplication::Get().RoutePointerDownEvent(WidgetPathUnderFinger, PointerEvent);
								}

								// In case of selecting a level in the content browser the VREditormode is closed, this makes sure nothing happens after that.
								if (!IVREditorModule::Get().IsVREditorModeActive())
								{
									bWasHandled = true;
									return;
								}

								VREditorInteractor->SetIsClickingOnUI(true);
								VREditorInteractor->SetIsRightClickingOnUI(bIsRightClicking);
								VREditorInteractor->SetLastUIPressTime(CurrentTime);
								bOutIsInputCaptured = true;

								// Play a haptic effect on press
								VREditorInteractor->PlayHapticEffect(VREd::UIPressHapticFeedbackStrength->GetFloat());
							}
							else if (Action.Event == IE_Released)
							{
								Reply = FSlateApplication::Get().RoutePointerUpEvent(WidgetPathUnderFinger, PointerEvent);
							}
						}
					}
				}
			}
		}
		if (Action.Event == IE_Released)
		{
			bool bWasRightClicking = false;
			if (VREditorInteractor->IsClickingOnUI())
			{
				if (VREditorInteractor->IsRightClickingOnUI())
				{
					bWasRightClicking = true;
				}
				VREditorInteractor->SetIsClickingOnUI(false);
				VREditorInteractor->SetIsRightClickingOnUI(false);
				bOutIsInputCaptured = false;
			}

			if (!bWasHandled)
			{
				TSet<FKey> PressedButtons;
				FPointerEvent PointerEvent(
					1 + (uint8)VREditorInteractor->GetControllerSide(),
					FVector2D::ZeroVector,
					FVector2D::ZeroVector,
					PressedButtons,
					bWasRightClicking ? EKeys::RightMouseButton : EKeys::LeftMouseButton,
					0.0f,	// Wheel delta
					FModifierKeysState());

				FWidgetPath EmptyWidgetPath;

				VREditorInteractor->SetIsClickingOnUI(false);
				VREditorInteractor->SetIsRightClickingOnUI(false);
				FReply Reply = FSlateApplication::Get().RoutePointerUpEvent(EmptyWidgetPath, PointerEvent);
			}
		}
	}
}


void UVREditorUISystem::OnVRAction( FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor,
	const FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled )
{

}

void UVREditorUISystem::OnVRHoverUpdate(UViewportInteractor* Interactor, FVector& HoverImpactPoint, bool& bWasHandled)
{
	static const FName SteamVR(TEXT("SteamVR"));
	UVREditorInteractor* VREditorInteractor = Cast<UVREditorInteractor>( Interactor );
	if( VREditorInteractor != nullptr )
	{
		if( !bWasHandled && Interactor->GetDraggingMode() == EViewportInteractionDraggingMode::Nothing )
		{
			FVector LaserPointerStart, LaserPointerEnd;
			if( Interactor->GetLaserPointer( LaserPointerStart, LaserPointerEnd ) )
			{
				FHitResult HitResult = Interactor->GetHitResultFromLaserPointer();
				if( HitResult.HitObjectHandle.IsValid() )
				{
					// The laser should make the quick radial menu stay active
					if ((HitResult.HitObjectHandle == QuickRadialMenu) && (QuickRadialMenu != nullptr))
					{
						RadialMenuModifierSpawnTime = FTimespan::FromSeconds(FPlatformTime::Seconds());
						FVector2D ReturnToCenter = FVector2D::ZeroVector;
						QuickRadialMenu->HighlightSlot(ReturnToCenter);
					}

					// Only allow clicks to our own widget components
					UVREditorWidgetComponent* WidgetComponent = Cast<UVREditorWidgetComponent>( HitResult.GetComponent() );
					if( WidgetComponent != nullptr )
					{
						FVector2D LastLocalHitLocation = WidgetComponent->GetLastLocalHitLocation();

						FVector2D LocalHitLocation;
						WidgetComponent->GetLocalHitLocation( HitResult.ImpactPoint, LocalHitLocation );

						// If we weren't already hovering over this widget, then we'll reset the last hit location
						if( WidgetComponent != VREditorInteractor->GetLastHoveredWidgetComponent() )
						{
							LastLocalHitLocation = LocalHitLocation;

							if ( UVREditorWidgetComponent* VRWidgetComponent = Cast<UVREditorWidgetComponent>( VREditorInteractor->GetLastHoveredWidgetComponent() ) )
							{
								VRWidgetComponent->SetIsHovering(false);
								OnHoverEndEffect(VRWidgetComponent);
							}
						}

						// @todo vreditor UI: There is a CursorRadius optional arg that we migth want to make use of wherever we call GetHitWidgetPath()!
						FWidgetPath WidgetPathUnderFinger = FWidgetPath( WidgetComponent->GetHitWidgetPath( HitResult.ImpactPoint, /*bIgnoreEnabledStatus*/ false ) );
						if ( WidgetPathUnderFinger.IsValid() )
						{
							HoverImpactPoint = HitResult.ImpactPoint;
							VREditorInteractor->GetInteractorData().LastHoverLocationOverUI = HitResult.ImpactPoint;
							VREditorInteractor->SetLastHoveredWidgetComponent( WidgetComponent );
							VREditorInteractor->SetIsHoveringOverUI( true );
							
							TSet<FKey> PressedButtons;
							if( VREditorInteractor->IsClickingOnUI() == true )
							{
								PressedButtons.Add( EKeys::LeftMouseButton );
							}
							else if( VREditorInteractor->IsRightClickingOnUI() == true )
							{
								PressedButtons.Add( EKeys::RightMouseButton );
							}

							FPointerEvent PointerEvent(
								1 + (uint8)VREditorInteractor->GetControllerSide(),
								LocalHitLocation,
								LastLocalHitLocation,
								LocalHitLocation - LastLocalHitLocation,
								PressedButtons,
								FModifierKeysState());

							FSlateApplication::Get().RoutePointerMoveEvent(WidgetPathUnderFinger, PointerEvent, false);
							
							bWasHandled = true;

							WidgetComponent->SetIsHovering(true);
							OnHoverBeginEffect(WidgetComponent);

							// Route the mouse scrolling
							if ( VREditorInteractor->IsTrackpadPositionValid( 1 ) )
							{
								const bool bIsAbsolute = ( GetOwner().GetHMDDeviceType() == SteamVR );

								float ScrollDelta = 0.0f;
								// Don't scroll if the radial menu is a number pad
								if(( VREditorInteractor->IsTouchingTrackpad() || !bIsAbsolute) && !bRadialMenuIsNumpad)
								{
									if( bIsAbsolute )
									{
										const float ScrollSpeed = VREd::UIAbsoluteScrollSpeed->GetFloat();
										ScrollDelta = (float) ( VREditorInteractor->GetTrackpadPosition().Y - VREditorInteractor->GetLastTrackpadPosition().Y ) * ScrollSpeed;
									}
									else
									{
										const float ScrollSpeed = VREd::UIRelativeScrollSpeed->GetFloat();
										ScrollDelta = (float) VREditorInteractor->GetTrackpadPosition().Y * ScrollSpeed;
									}
								}

								// If using a trackpad (Vive), invert scroll direction so that it feels more like scrolling on a mobile device
								if( GetOwner().GetHMDDeviceType() == SteamVR )
								{
									ScrollDelta *= -1.0f;
								}

								if( !FMath::IsNearlyZero( ScrollDelta ) )
								{
									FPointerEvent MouseWheelEvent(
										1 + (uint8)VREditorInteractor->GetControllerSide(),
										LocalHitLocation,
										LastLocalHitLocation,
										PressedButtons,
										EKeys::MouseWheelAxis,
										ScrollDelta,
										FModifierKeysState() );

									FSlateApplication::Get().RouteMouseWheelOrGestureEvent(WidgetPathUnderFinger, MouseWheelEvent, nullptr);

									VREditorInteractor->SetUIScrollVelocity( 0.0f );

									// Don't apply inertia unless the user dragged a decent amount this frame
									if( bIsAbsolute && FMath::Abs( ScrollDelta ) >= VREd::MinUIScrollDeltaForInertia->GetFloat() )
									{
										// Don't apply inertia if our data is sort of old
										const FTimespan CurrentTime = FTimespan::FromSeconds( FPlatformTime::Seconds() );
										if( CurrentTime - VREditorInteractor->GetLastTrackpadPositionUpdateTime() < FTimespan::FromSeconds( 1.0f / 30.0f ) )
										{
											//GWarn->Logf( TEXT( "INPUT: UIScrollVelocity=%0.2f" ), ScrollDelta );
											VREditorInteractor->SetUIScrollVelocity( ScrollDelta );
										}
									}
								}
							}
							else
							{
								if( !FMath::IsNearlyZero( VREditorInteractor->GetUIScrollVelocity() ) )
								{
									// Apply UI scrolling inertia
									const float ScrollDelta = VREditorInteractor->GetUIScrollVelocity();
									{
										FPointerEvent MouseWheelEvent(
											1 + (uint8)VREditorInteractor->GetControllerSide(),
											LocalHitLocation,
											LastLocalHitLocation,
											PressedButtons,
											EKeys::MouseWheelAxis,
											ScrollDelta,
											FModifierKeysState() );

										FSlateApplication::Get().RouteMouseWheelOrGestureEvent( WidgetPathUnderFinger, MouseWheelEvent, nullptr );
									}

									// Apply damping
									FVector ScrollVelocityVector( VREditorInteractor->GetUIScrollVelocity(), 0.0f, 0.0f );
									const bool bVelocitySensitive = false;
									GetOwner().GetWorldInteraction().ApplyVelocityDamping( ScrollVelocityVector, bVelocitySensitive );
									VREditorInteractor->SetUIScrollVelocity( (float)ScrollVelocityVector.X );
								}
								else
								{
									VREditorInteractor->SetUIScrollVelocity( 0.0f );
								}
							}
						}
					}
				}
			}
		}

		// If nothing was hovered, make sure we tell Slate about that
		if( !bWasHandled && VREditorInteractor->GetLastHoveredWidgetComponent() != nullptr )
		{
			if ( UVREditorWidgetComponent* VRWidgetComponent = Cast<UVREditorWidgetComponent>( VREditorInteractor->GetLastHoveredWidgetComponent() ) )
			{
				VRWidgetComponent->SetIsHovering( false );
				OnHoverEndEffect(VRWidgetComponent);
			}

			const FVector2D LastLocalHitLocation = VREditorInteractor->GetLastHoveredWidgetComponent()->GetLastLocalHitLocation();
			VREditorInteractor->SetLastHoveredWidgetComponent( nullptr );

			TSet<FKey> PressedButtons;
			FPointerEvent PointerEvent(
				1 + (uint8)VREditorInteractor->GetControllerSide(),
				LastLocalHitLocation,
				LastLocalHitLocation,
				FVector2D::ZeroVector,
				PressedButtons,
				FModifierKeysState() );


			FSlateApplication::Get().RoutePointerMoveEvent( FWidgetPath(), PointerEvent, false );
		}

	}

}

void UVREditorUISystem::Tick( FEditorViewportClient* ViewportClient, const float DeltaTime )
{
	if ( bRefocusViewport )
	{
		bRefocusViewport = false;
		FSlateApplication::Get().SetUserFocus(0, ViewportClient->GetEditorViewportWidget());
	}

	const FTimespan CompareTime = FMath::Max(UIInteractor->GetLastActiveTrackpadUpdateTime(), RadialMenuModifierSpawnTime);
	if (!bRadialMenuIsNumpad &&
		FTimespan::FromSeconds(FPlatformTime::Seconds()) - CompareTime > FTimespan::FromSeconds(1.5f))
	{
		if (IsShowingRadialMenu( UIInteractor ))
		{
			HideRadialMenu();
		}
	}

	// When dragging a panel with the UI interactor for a while, we want to close the radial menu.
	if (bDragPanelFromOpen)
	{
		DragPanelFromOpenTime += DeltaTime;
		if (DragPanelFromOpenTime >= 1.0 && UIInteractor != nullptr && DraggingUI != nullptr && InteractorDraggingUI != nullptr && UIInteractor == InteractorDraggingUI)
		{
			HideRadialMenu(false);
		}
	}


	
	// Iterate through all quick menu and radial menu buttons and animate any that need it
	for (FVRButton& VRButton : VRButtons)
	{
		switch (VRButton.AnimationDirection)
		{
			case EVREditorAnimationState::Forward:
			{
				if (VRButton.CurrentScale < VRButton.MaxScale)
				{
					VRButton.CurrentScale += DeltaTime*VRButton.ScaleRate;
				}
				else
				{
					VRButton.CurrentScale = VRButton.MaxScale;
					VRButton.AnimationDirection = EVREditorAnimationState::None;
				}
				break;
			}
			case EVREditorAnimationState::Backward:
			{
				if (VRButton.CurrentScale > VRButton.MinScale)
				{
					VRButton.CurrentScale -= DeltaTime*VRButton.ScaleRate;
				}
				else
				{
					VRButton.CurrentScale = VRButton.MinScale;
					VRButton.AnimationDirection = EVREditorAnimationState::None;	
				}
				break;
			}
			case EVREditorAnimationState::None:
			{
				break;
			}
		}
		if (VRButton.ButtonWidget != nullptr)
		{
			VRButton.ButtonWidget->SetRelativeScale3D(VRButton.CurrentScale*VRButton.OriginalRelativeScale*(GetOwner().GetWorldScaleFactor()));
		}
	}
	
	// Tick all of our floating UIs
	for(auto CurrentUI : FloatingUIs)
	{
		AVREditorFloatingUI* UIPanel = CurrentUI.Value;
		if (UIPanel != nullptr)
		{
			UIPanel->TickManually( DeltaTime );
		}
	}
	QuickRadialMenu->TickManually(DeltaTime);
}

void UVREditorUISystem::Render( const FSceneView* SceneView, FViewport* Viewport, FPrimitiveDrawInterface* PDI )
{
	// ...
}

void UVREditorUISystem::CreateUIs()
{
	static const FName SteamVR(TEXT("SteamVR"));
	static const FName OculusHMD(TEXT("OculusHMD"));

	const FIntPoint DefaultResolution( VREd::DefaultEditorUIResolutionX->GetInt(), VREd::DefaultEditorUIResolutionY->GetInt() );
	const bool bShowUI = UVREditorMode::IsDebugModeEnabled();

	{
		const bool bWithSceneComponent = false;

		// @todo vreditor: Tweak
		{
			InfoDisplayPanel = GetOwner().SpawnTransientSceneActor<AVREditorFloatingUI>(TEXT("QuickMenu"), bWithSceneComponent);
			const FIntPoint Resolution(512, 64);
			InfoDisplayPanel->SetSlateWidget(*this, InfoDisplayPanelID, SNullWidget::NullWidget, Resolution, 20.0f, AVREditorBaseActor::EDockedTo::Nothing);
			InfoDisplayPanel->ShowUI(bShowUI);
			FVector RelativeOffset = VRMode->GetHMDDeviceType() == SteamVR ?  FVector(5.0f, 0.0f, 0.0f) : FVector(5.0f, 0.0f, 10.0f);
			InfoDisplayPanel->SetRelativeOffset(RelativeOffset);
			InfoDisplayPanel->SetWindowMesh(VRMode->GetAssetContainer().WindowMesh);
			FloatingUIs.Add(InfoDisplayPanelID, InfoDisplayPanel);
		}

		// Create the radial UI
		{
			QuickRadialMenu = GetOwner().SpawnTransientSceneActor<AVREditorRadialFloatingUI>(TEXT("QuickRadialmenu"), bWithSceneComponent);
			FVector RelativeOffset = FVector::ZeroVector;
			if (VRMode->GetHMDDeviceType() == SteamVR)
			{
				RelativeOffset = FVector(-5.0f, 0.0f, 5.0f);
			}
			else if (VRMode->GetHMDDeviceType() == OculusHMD)
			{
				RelativeOffset = FVector(0.0f, 0.0f, 3.0f);
			}

			QuickRadialMenu->SetRelativeOffset(RelativeOffset);
			QuickRadialMenu->ShowUI(bShowUI, false, 0.0f, false);
		}
	}
	// Make some editor UIs!
	{
		{
			const FName TabIdentifier = NAME_None;	// No tab for us!
			const TSharedRef<SWidget> SequencerWidget = SNullWidget::NullWidget;
			
			TSharedRef<SWidget> WidgetToDraw =
				SNew(SDPIScaler)
				.DPIScale(VREd::EditorUIScale->GetFloat())
				[
					SequencerWidget
				]
			;

			const bool bWithSceneComponent = false;
			AVREditorFloatingUI* SequencerUI = GetOwner().SpawnTransientSceneActor<AVREditorDockableWindow>(TEXT("SequencerUI"), bWithSceneComponent);
			SequencerUI->SetSlateWidget(*this, SequencerPanelID, WidgetToDraw, FIntPoint((int32)VREd::SequencerUIResolutionX->GetFloat(), (int32)VREd::SequencerUIResolutionY->GetFloat()), VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
			SequencerUI->ShowUI(false);
			FloatingUIs.Add(SequencerPanelID, SequencerUI);
		}
	}
}

void UVREditorUISystem::OnAssetEditorOpened(UObject* Asset)
{
	
	{
		// We need to disable drag drop on the tabs spawned in VR mode.
		TArray<IAssetEditorInstance*> Editors = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorsForAsset(Asset);
		for ( IAssetEditorInstance* Editor : Editors )
		{
			if (Editor->GetAssociatedTabManager().IsValid())
			{
				Editor->GetAssociatedTabManager()->SetCanDoDragOperation(false);
			}
			else
			{
				FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
				LevelEditorModule.GetLevelEditorTabManager().ToSharedRef()->SetCanDoDragOperation(false);
			}
		}
	}
}

bool UVREditorUISystem::IsShowingEditorUIPanel(const VREditorPanelID& InPanelID) const
{
	AVREditorFloatingUI* Panel = GetPanel(InPanelID);
	if( Panel != nullptr )
	{
		return Panel->IsUIVisible();
	}

	return false;
}

void UVREditorUISystem::ShowEditorUIPanel(const UWidgetComponent* WidgetComponent, UVREditorInteractor* Interactor, const bool bShouldShow, const bool bSpawnInFront, const bool bDragFromOpen, const bool bPlaySound)
{
	AVREditorFloatingUI* ResultPanel = nullptr;
	for (auto& CurrentUI : FloatingUIs)
	{
		AVREditorFloatingUI* Panel = CurrentUI.Value;
		if (Panel != nullptr && Panel->GetWidgetComponent() == WidgetComponent)
		{
			ResultPanel = Panel;
			break;
		}
	}

	ShowEditorUIPanel(ResultPanel, Interactor, bShouldShow, bSpawnInFront, bDragFromOpen, bPlaySound);
}

void UVREditorUISystem::ShowEditorUIPanel(const VREditorPanelID& InPanelID, UVREditorInteractor* Interactor, const bool bShouldShow, const bool bSpawnInFront, const bool bDragFromOpen, const bool bPlaySound)
{
	AVREditorFloatingUI* Panel = GetPanel(InPanelID);
	ShowEditorUIPanel(Panel, Interactor, bShouldShow, bSpawnInFront, bDragFromOpen, bPlaySound);
}

void UVREditorUISystem::ShowEditorUIPanel(AVREditorFloatingUI* Panel, UVREditorInteractor* Interactor, const bool bShouldShow, const bool bSpawnInFront, const bool bDragFromOpen, const bool bPlaySound)
{
	if (Panel != nullptr)
	{
		AVREditorFloatingUI::EDockedTo DockedTo = Panel->GetDockedTo();
		if (bShouldShow && (bSpawnInFront || DockedTo == AVREditorFloatingUI::EDockedTo::Nothing || bDragFromOpen))
		{
			Panel->SetScale(Panel->GetInitialScale(), false);

			// Make sure to set the panel to dock to the room if it is docked to nothing. Otherwise it will show up in world space.
			if (DockedTo == AVREditorFloatingUI::EDockedTo::Nothing && Panel->CreationContext.ParentActor == nullptr)
			{
				Panel->SetDockedTo(AVREditorBaseActor::EDockedTo::Room);
			}
		
			const float OpenDistance = VREd::UIPanelOpenDistance->GetFloat() * GetOwner().GetWorldScaleFactor();
			FTransform WindowToWorldTransform;
			bool bOpenAtControllerPosition = Panel->CreationContext.PanelSpawnOffset.Equals(FTransform::Identity);
						
			if (bOpenAtControllerPosition) // This is the default behavior that will trigger for all windows opened via the radial menu
			{
				// Set the initial transform when opening a panel.
				FTransform HandTransform;
				FVector HandForward;
				Interactor->GetTransformAndForwardVector(HandTransform, HandForward);
				
				WindowToWorldTransform.SetLocation(HandTransform.GetLocation() + (HandForward.GetSafeNormal() * OpenDistance));				
				const FQuat Rotation = (HandTransform.GetLocation() - WindowToWorldTransform.GetLocation()).ToOrientationQuat() * FRotator(VREd::UIPanelOpenRotationPitchOffset->GetFloat(), 0.f, 0.f).Quaternion();
				WindowToWorldTransform.SetRotation(Rotation);
			}
			else // Open at hardcoded offset, axis-aligned, in front of headset. Triggered via blueprint node in VirtualProduction plugin
			{
				FTransform EditorHeadTransform = VRMode->GetWorldInteraction().GetHeadTransform();							
				FRotator EditorHeadRotationAxisAligned = EditorHeadTransform.GetRotation().Rotator();
				EditorHeadRotationAxisAligned.Pitch = 0;
				EditorHeadRotationAxisAligned.Roll = 0;
				FQuat FinishedHMDRotator(EditorHeadRotationAxisAligned);
				
				FVector NewLocation = EditorHeadTransform.GetLocation() + (FinishedHMDRotator.GetForwardVector().GetSafeNormal() * Panel->CreationContext.PanelSpawnOffset.GetLocation().X);
				NewLocation += FinishedHMDRotator.GetRightVector().GetSafeNormal() * Panel->CreationContext.PanelSpawnOffset.GetLocation().Y;
				NewLocation += FinishedHMDRotator.GetUpVector().GetSafeNormal() * Panel->CreationContext.PanelSpawnOffset.GetLocation().Z;
				WindowToWorldTransform.SetLocation(NewLocation);																
				WindowToWorldTransform.SetRotation(FinishedHMDRotator);
			}
	
			if (bDragFromOpen)
			{
				AVREditorDockableWindow* DockableWindow = Cast<AVREditorDockableWindow>(Panel);
				if (DockableWindow != nullptr)
				{
					bDragPanelFromOpen = true;
					DragPanelFromOpenTime = 0.0f;
					bPanelCanScale = false;

					DockableWindow->SetActorTransform(WindowToWorldTransform);
					DockableWindow->SetDockSelectDistance(OpenDistance);

					const bool bPlayStartDragSound = false;
					StartDraggingDockUI(DockableWindow, Interactor, OpenDistance, bPlayStartDragSound);
					VRMode->GetWorldInteraction().SetDraggedInteractable(DockableWindow, Interactor);
				}
			}
			//Attach to parent actor if it has been assigned
			else if (Panel->CreationContext.ParentActor != nullptr)
			{
				FVector TransformedOffset = Panel->CreationContext.ParentActor->GetTransform().TransformVector(Panel->CreationContext.PanelSpawnOffset.GetLocation());
				FVector NewLocation = Panel->CreationContext.ParentActor->GetActorLocation() + TransformedOffset;
				FQuat NewRotation = Panel->CreationContext.ParentActor->GetActorRotation().Quaternion() * Panel->CreationContext.PanelSpawnOffset.GetRotation();
				Panel->SetActorLocationAndRotation(NewLocation, NewRotation);
				Panel->AttachToActor(Panel->CreationContext.ParentActor, FAttachmentTransformRules::KeepWorldTransform);
			}
			//else attach to room
			else
			{
				if (DockedTo != AVREditorFloatingUI::EDockedTo::Room)
				{
					Panel->SetDockedTo(AVREditorBaseActor::EDockedTo::Room);
				}

				const FTransform RoomToWorld = GetOwner().GetRoomTransform();
				const FTransform WorldToRoom = RoomToWorld.Inverse();
				const FTransform WindowToRoomTransform = WindowToWorldTransform * WorldToRoom;
				const FVector RoomSpaceWindowLocation = WindowToRoomTransform.GetLocation() / GetOwner().GetWorldScaleFactor();
				const FQuat RoomSpaceWindowRotation = WindowToRoomTransform.GetRotation();
				Panel->SetRelativeOffset(RoomSpaceWindowLocation);
				FRotator Rotation;

				if (bOpenAtControllerPosition)
				{
					Rotation = RoomSpaceWindowRotation.Rotator();
				}
				else
				{
					Rotation = WindowToRoomTransform.Rotator();
					Rotation.Yaw += 180;
					Rotation.Yaw += Panel->CreationContext.PanelSpawnOffset.GetRotation().Rotator().Yaw;
				}

				Panel->SetLocalRotation(Rotation);
			}
		}

		// Handle special cases for closing a panel.
		if (!bShouldShow)
		{
			// If we are closing the sequencer panel, then also null out the sequencer widget and close the Sequencer instance
			const VREditorPanelID& PanelID = Panel->GetID();
			if (PanelID == SequencerPanelID)
			{
				if (GetOwner().GetCurrentSequencer() != nullptr)
				{
					Panel->SetSlateWidget(*this, SequencerPanelID, SNullWidget::NullWidget, FIntPoint((int32)VREd::SequencerUIResolutionX->GetFloat(), (int32)VREd::SequencerUIResolutionY->GetFloat()), VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
					FVREditorActionCallbacks::CloseSequencer(GetOwner().GetCurrentSequencer()->GetRootMovieSceneSequence());
				}
			}
			else if (PanelID.ToString().Contains(ActorPreviewPrefix))
			{
				// if we are closing a preview panel, make sure to unpin it also
				TObjectPtr<AActor>* PreviewedActorPtr = PreviewWindowInfo.Find(PanelID);
				if (PreviewedActorPtr != nullptr)
				{
					AActor* PreviewedActor = *PreviewedActorPtr;
					if (PreviewedActor && GetOwner().GetLevelViewportPossessedForVR().IsActorPreviewPinned(PreviewedActor))
					{
						GetOwner().GetLevelViewportPossessedForVR().ToggleActorPreviewIsPinned(PreviewedActor);
					}
				}
			}
		}

		Panel->ShowUI(bShouldShow);

		if (bPlaySound)
		{
			const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();
			VRMode->PlaySound(bShouldShow ? AssetContainer.DockableWindowOpenSound : AssetContainer.DockableWindowCloseSound, Panel->GetActorLocation());
		}
	}
}


bool UVREditorUISystem::IsShowingRadialMenu(const UVREditorInteractor* Interactor ) const
{
	if (QuickRadialMenu != nullptr)
	{
		const EControllerHand DockedToHand = QuickRadialMenu->GetDockedTo() == AVREditorFloatingUI::EDockedTo::LeftArm ? EControllerHand::Left : EControllerHand::Right;
		const UVREditorInteractor* DockedToInteractor = GetOwner().GetHandInteractor(DockedToHand);
		return DockedToInteractor == Interactor && !QuickRadialMenu->IsHidden();
	}
	else
	{
		return false;
	}
}


void UVREditorUISystem::TryToSpawnRadialMenu( UVREditorInteractor* Interactor, const bool bForceRefresh, const bool bPlaySound /*= true*/ )
{
	if( Interactor != nullptr && Interactor->GetControllerType() == EControllerType::UI)
	{
		UVREditorInteractor* MotionControllerInteractor = Cast<UVREditorInteractor>( Interactor );
		if( MotionControllerInteractor != nullptr )
		{

			EViewportInteractionDraggingMode DraggingMode = Interactor->GetDraggingMode();

			bool bNeedsSpawn = (bForceRefresh || !QuickRadialMenu->IsUIVisible()) &&
				MotionControllerInteractor == UIInteractor && 
				DraggingMode != EViewportInteractionDraggingMode::TransformablesAtLaserImpact &&	 // Don't show radial menu if the hand is busy dragging something around
				DraggingMode != EViewportInteractionDraggingMode::TransformablesFreely &&
				DraggingMode != EViewportInteractionDraggingMode::World &&
				DraggingMode != EViewportInteractionDraggingMode::AssistingDrag &&
				bDragPanelFromOpen == false;

			if (bNeedsSpawn)
			{
				RadialMenuModifierSpawnTime = FTimespan::FromSeconds(FPlatformTime::Seconds());
				if(bRadialMenuIsNumpad)
				{ 
					BuildNumPadWidget();
				}
				else
				{
					BuildRadialMenuWidget();
				}
				const AVREditorFloatingUI::EDockedTo DockedTo = UIInteractor->GetControllerSide() == EControllerHand::Left ? AVREditorFloatingUI::EDockedTo::LeftArm : AVREditorFloatingUI::EDockedTo::RightArm;
				QuickRadialMenu->SetDockedTo(DockedTo);
				QuickRadialMenu->ShowUI(true);
				Interactor->TryOverrideControllerType(EControllerType::UI);
			}
		}
	}
}


void UVREditorUISystem::HideRadialMenu(const bool bPlaySound /*= true */, const bool bAllowFading /*= true*/)
{
	// Only hide the radial menu if the passed interactor is actually the interactor with the radial menu
	if( IsShowingRadialMenu( UIInteractor ))
	{
		QuickRadialMenu->ShowUI( false, bAllowFading, VREd::RadialMenuFadeDelay->GetFloat(), bPlaySound );
	}
	UIInteractor->TryOverrideControllerType(EControllerType::Unknown);
}


FTransform UVREditorUISystem::MakeDockableUITransformOnLaser( AVREditorDockableWindow* InitDraggingDockUI, UVREditorInteractor* Interactor, const float DockSelectDistance ) const
{
	FTransform HandTransform;
	FVector HandForward;
	Interactor->GetTransformAndForwardVector( HandTransform, HandForward );
	FTransform InteractorTransform = Interactor->GetTransform();

	// Use the smoothed laser pointer direction for computing an offset direction.  It looks a lot better!
	FVector LaserPointerStart, LaserPointerEnd;
	const bool bEvenIfBlocked = true;
	Interactor->GetLaserPointer( /* Out */ LaserPointerStart, /* Out */ LaserPointerEnd, bEvenIfBlocked );

	const FVector LaserPointerDirection = ( LaserPointerEnd - LaserPointerStart ).GetSafeNormal();
	const FVector NewLocation = InteractorTransform.GetLocation() + ( LaserPointerDirection * DockSelectDistance );

	const FTransform LaserImpactToWorld( InteractorTransform.GetRotation(), NewLocation );
	return LaserImpactToWorld;
}


FTransform UVREditorUISystem::MakeDockableUITransform( AVREditorDockableWindow* InitDraggingDockUI, UVREditorInteractor* Interactor, const float DockSelectDistance )
{
	const FTransform LaserImpactToWorld = MakeDockableUITransformOnLaser( DraggingUI, Interactor, DockSelectDistance );
	const FTransform UIToLaserImpact = DraggingUIOffsetTransform;
	const FTransform UIToWorld = UIToLaserImpact * LaserImpactToWorld;

	// Enable the ability to scale the UI when opening from the radial menu after the trackpad was reset.
	if (!bPanelCanScale && bDragPanelFromOpen && UIInteractor != nullptr && DraggingUI != nullptr && InteractorDraggingUI != nullptr && UIInteractor == InteractorDraggingUI)
	{
		UVREditorInteractor* MotionController = Cast<UVREditorInteractor>(InteractorDraggingUI);
		if (MotionController != nullptr)
		{
			if (MotionController->GetTrackpadPosition().IsNearlyZero(0.1f))
			{
				bPanelCanScale = true;
			}
		}
	}

	return UIToWorld;
}


void UVREditorUISystem::StartDraggingDockUI(AVREditorDockableWindow* InitDraggingDockUI, UVREditorInteractor* Interactor, const float DockSelectDistance, const bool bPlaySound /*= true*/)
{
	InteractorDraggingUI = Interactor;
	FTransform UIToWorld = InitDraggingDockUI->GetActorTransform();
	UIToWorld.SetScale3D( FVector( 1.0f ) );
	const FTransform WorldToUI = UIToWorld.Inverse();

	const FTransform LaserImpactToWorld = MakeDockableUITransformOnLaser( InitDraggingDockUI, Interactor, DockSelectDistance );
	const FTransform LaserImpactToUI = LaserImpactToWorld * WorldToUI;
	const FTransform UIToLaserImpact = LaserImpactToUI.Inverse();
	DraggingUIOffsetTransform = UIToLaserImpact;

	DraggingUI = InitDraggingDockUI;
	DraggingUI->SetDockedTo( AVREditorFloatingUI::EDockedTo::Dragging );

	if (bPlaySound)
	{
		VRMode->PlaySound(VRMode->GetAssetContainer().DockableWindowDragSound, LaserImpactToWorld.GetLocation()); //@todo VREditor: Removed sounds here and put in DockableWindow
	}
}

AVREditorDockableWindow* UVREditorUISystem::GetDraggingDockUI() const
{
	return DraggingUI;
}

void UVREditorUISystem::StopDraggingDockUI( UVREditorInteractor* VREditorInteractor, const bool bPlaySound /*= true*/ )
{
	if ( IsInteractorDraggingDockUI( VREditorInteractor ) )
	{
		// Put the Dock back on the hand it came from or leave it where it is in the room
		UViewportInteractor* OtherInteractor = VREditorInteractor->GetOtherInteractor();
		if ( OtherInteractor )
		{
			UVREditorInteractor* MotionControllerOtherInteractor = Cast<UVREditorInteractor>( OtherInteractor );
			if ( MotionControllerOtherInteractor )
			{
				// Reset the the panel that we were dragging to room space.
				DraggingUI->SetDockedTo( AVREditorFloatingUI::EDockedTo::Room );

				// Play stop dragging sound.
				if (bPlaySound)
				{
					VRMode->PlaySound(VRMode->GetAssetContainer().DockableWindowDropSound, DraggingUI->GetActorLocation());
				}

				// We are not dragging anymore.
				DraggingUI = nullptr;
				InteractorDraggingUI->SetDraggingMode( EViewportInteractionDraggingMode::Nothing );
				InteractorDraggingUI = nullptr;
			}
		}
	}
}

bool UVREditorUISystem::IsDraggingDockUI()
{
	return DraggingUI != nullptr && DraggingUI->GetDockedTo() == AVREditorFloatingUI::EDockedTo::Dragging;
}

bool UVREditorUISystem::IsInteractorDraggingDockUI( const UVREditorInteractor* Interactor ) const
{
	return InteractorDraggingUI && InteractorDraggingUI == Interactor;
}

bool UVREditorUISystem::IsDraggingPanelFromOpen() const
{
	return bDragPanelFromOpen;
}

void UVREditorUISystem::TogglePanelsVisibility()
{
	bool bAnyPanelsVisible = false;

	// Check if there is any panel visible and if any is docked to a hand
	for (auto& CurrentUI : FloatingUIs)
	{
		AVREditorFloatingUI* Panel = CurrentUI.Value;
		if (Panel && Panel->IsUIVisible())
		{
			bAnyPanelsVisible = true;
			break;
		}
	}

	// Hide if there is any UI visible
	const bool bShowUI = !bAnyPanelsVisible;

	for (auto& CurrentUI : FloatingUIs)
	{
		AVREditorFloatingUI* Panel = CurrentUI.Value;
		if (Panel != nullptr && Panel->IsUIVisible() != bShowUI)
		{
			Panel->ShowUI(bShowUI);
		}
	}

	if (UIInteractor)
	{
		// Play sound
		const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();
		VRMode->PlaySound(bShowUI ? AssetContainer.DockableWindowOpenSound : AssetContainer.DockableWindowCloseSound, UIInteractor->GetTransform().GetLocation());
	}
}

float UVREditorUISystem::GetMaxDockWindowSize() const
{
	return VREd::MaxDockWindowSize->GetFloat();
}

float UVREditorUISystem::GetMinDockWindowSize() const
{
	return VREd::MinDockWindowSize->GetFloat();
}

void UVREditorUISystem::IsProxyTabSupported( FTabId TabId, bool& bOutIsTabSupported )
{
	// Avoid spawning GPU Profiler tab into VR
	static FName VisualizerSpawnPointTabId( "VisualizerSpawnPoint" );
	if( TabId.TabType == VisualizerSpawnPointTabId )
	{
		bOutIsTabSupported = false;
	}
}

void UVREditorUISystem::OnProxyTabLaunched(TSharedPtr<SDockTab> NewTab)
{
	ShowAssetEditor();
}

void UVREditorUISystem::OnAttentionDrawnToTab(TSharedPtr<SDockTab> NewTab)
{
	// @todo vreditor: When clicking on Modes icons in the Modes panel, the LevelEditor tab is invoked which causes an empty asset editor to pop-up
	static FName LevelEditorTabID( "LevelEditor" );
	if( NewTab->GetLayoutIdentifier().TabType != LevelEditorTabID )
	{
		ShowAssetEditor();
	}
}

void UVREditorUISystem::ShowAssetEditor()
{
	bRefocusViewport = true;

	// A tab was opened, so make sure the "Asset" UI is visible.  That's where the user can interact
	// with the newly-opened tab
	AVREditorFloatingUI* AssetEditorPanel = GetPanel(TabManagerPanelID);
	if (AssetEditorPanel != nullptr && !AssetEditorPanel->IsUIVisible())
	{	
		const bool bShouldShow = true;
		const bool bSpawnInFront = true;
		ShowEditorUIPanel(AssetEditorPanel, UIInteractor, bShouldShow, bSpawnInFront);

		// Play haptic effect so user knows to look at their hand that now has UI on it!
		UIInteractor->PlayHapticEffect( VREd::UIAssetEditorSummonedOnHandHapticFeedbackStrength->GetFloat() );
	}
}

void UVREditorUISystem::TogglePanelVisibility(const VREditorPanelID& InPanelID)
{
	AVREditorFloatingUI* Panel = GetPanel(InPanelID);
	if (Panel != nullptr)
	{
		const bool bIsShowing = Panel->IsUIVisible();
		if(bIsShowing)
		{
			Panel->ShowUI(false);
		}
		else
		{
			const bool bDragFromOpen = ShouldPreviewPanel();
			const bool bSpawnInFront = true;
			const bool bPlaySound = false;
			ShowEditorUIPanel(Panel, UIInteractor, !bIsShowing, bSpawnInFront, bDragFromOpen, bPlaySound);
		}
	}
}

void UVREditorUISystem::DockableAreaRestored()
{
}



void UVREditorUISystem::MakeUniformGridMenu(const TSharedRef<FMultiBox>& MultiBox, const TSharedRef<SMultiBoxWidget>& MultiBoxWidget, int32 Columns)
{
	// Grab the list of blocks, early out if there's nothing to fill the widget with
	const TArray< TSharedRef<const FMultiBlock > >& Blocks = MultiBox->GetBlocks();
	const uint32 NumItems = Blocks.Num();
	if( NumItems == 0 )
	{
		return;
	}

	TSharedRef<SUniformGridPanel> GridPanel = SNew( SUniformGridPanel )
		.SlotPadding(3.0f);
	MultiBox.Get().SetStyle(&FVREditorStyle::Get(), FVREditorStyle::GetStyleSetName());
	int32 Index = 0;
	int32 Row = 0;
	for( const TSharedRef<const FMultiBlock>& MultiBlock : Blocks )
	{
		const TSharedRef<const FMultiBlock>& Block = MultiBlock;
		if(Block->GetType() == EMultiBlockType::MenuEntry)
		{
			TSharedRef<SWidget> BlockWidget = Block->MakeWidget(MultiBoxWidget, EMultiBlockLocation::Middle, true, nullptr)->AsWidget();
		
			int32 Column = (int32) FMath::Fmod(static_cast<float>(Index), static_cast<float>(Columns));
			Row = FMath::DivideAndRoundDown(Index, Columns);
			TSharedRef<SOverlay> TestOverlay = SNew(SOverlay);
			GridPanel->AddSlot(Column, Row)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(TestOverlay, SOverlay)
				];

			//AddHoverableButton(BlockWidget, TEXT("SMenuEntryButton"), TestOverlay);
			//SetButtonTextWrap(BlockWidget, 50.0);
			Index++;
		}
	}

	MultiBoxWidget->SetContent(
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		.Padding(0, 0, 0, 0)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			GridPanel
		]
	);
}

TSharedRef<SWidget> UVREditorUISystem::AddHoverableButton(TSharedRef<SWidget>& BlockWidget, FName ButtonType, TSharedRef<SOverlay>& TestOverlay)
{

	TSharedRef<SWidget> TestWidget = FindWidgetOfType(BlockWidget, ButtonType);
	if (TestWidget != SNullWidget::NullWidget)
	{
		TSharedRef<SButton> Button = StaticCastSharedRef<SButton>(TestWidget);
		Button->SetRenderTransformPivot(FVector2D(0.5, 0.5));
		FSlateSound SlateButtonPressSound = FSlateSound();

		const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();
		SlateButtonPressSound.SetResourceObject(AssetContainer.ButtonPressSound);
		Button->SetPressedSound(SlateButtonPressSound);
	}
	return BlockWidget;
}

TSharedRef<SWidget> UVREditorUISystem::SetButtonFormatting(TSharedRef<SWidget>& BlockWidget, float WrapSize)
{
	TSharedRef<SWidget> TestWidget = FindWidgetOfType(BlockWidget, FName(TEXT("SImage")));
	if (TestWidget != SNullWidget::NullWidget)
	{
		TSharedRef<SImage> Image = StaticCastSharedRef<SImage>(TestWidget);
		Image->SetRenderTransformPivot(FVector2D(0.5f, 0.5f));
		Image->SetRenderTransform(FTransform2D(4.0f));	
	}

	// Format the button text
	TestWidget = FindWidgetOfType(BlockWidget, FName(TEXT("STextBlock")));
	if (TestWidget != SNullWidget::NullWidget)
	{
		TSharedRef<STextBlock> TextBlock = StaticCastSharedRef<STextBlock>(TestWidget);
		TextBlock->SetWrapTextAt(WrapSize);
		TextBlock->SetJustification(ETextJustify::Center);
		
		// Change the button font color based on whether the option is checked or not
		TSharedRef<SWidget> TestCheckbox = FindWidgetOfType(BlockWidget, FName(TEXT("SCheckBox")));
		if (TestCheckbox != SNullWidget::NullWidget)
		{
			TSharedRef<SCheckBox> CheckBox = StaticCastSharedRef<SCheckBox>(TestCheckbox);
			TAttribute<FSlateColor> DynamicSelectedTextColor = TAttribute<FSlateColor>::Create(
				TAttribute<FSlateColor>::FGetter::CreateLambda([this, CheckBox]() -> FSlateColor
				{
					return CheckBox->IsChecked() ? this->GetOwner().GetColor(UVREditorMode::EColors::UIColor) : FSlateColor(FLinearColor::White);
				}
				)
				);
				TAttribute<FSlateFontInfo> DynamicSelectedTextFont = TAttribute<FSlateFontInfo>::Create(
					TAttribute<FSlateFontInfo>::FGetter::CreateLambda([CheckBox]() -> FSlateFontInfo
				{
					return CheckBox->IsChecked() ? FVREditorStyle::Get().GetFontStyle("VRRadialStyle.ActiveFont") : FVREditorStyle::Get().GetFontStyle("VRRadialStyle.InactiveFont");
				}
				)
				);
				TextBlock->SetColorAndOpacity(DynamicSelectedTextColor);
				TextBlock->SetFont(DynamicSelectedTextFont);
			}
		

	}	
	return BlockWidget;
}

void UVREditorUISystem::MakeRadialBoxMenu(const TSharedRef<FMultiBox>& MultiBox, const TSharedRef<SMultiBoxWidget>& MultiBoxWidget, float RadiusRatioOverride, FName ButtonTypeOverride)
{
	// Grab the list of blocks, early out if there's nothing to fill the widget with
	const TArray< TSharedRef< const FMultiBlock > >& Blocks = MultiBox->GetBlocks();
	const uint32 NumItems = Blocks.Num();
	if (NumItems == 0)
	{
		return;
	}

	FName StyleName = NAME_None;
	if (!QuickRadialMenu->GetCurrentMenuWidget().IsValid() || MultiBoxWidget != QuickRadialMenu->GetCurrentMenuWidget())
	{
		if (bRadialMenuIsNumpad)
		{
			const FName NumPadOverride = TEXT("SButton");
			QuickRadialMenu->SetButtonTypeOverride(NumPadOverride);
			StyleName = FVREditorStyle::GetNumpadStyleSetName();
		}
		else
		{
			const FName RadialMenuOverride = TEXT("SMenuEntryButton");
			QuickRadialMenu->SetButtonTypeOverride(RadialMenuOverride);
			StyleName = FVREditorStyle::GetSecondaryStyleSetName();
		}
		QuickRadialMenu->Reset();
		QuickRadialMenu->SetCurrentMenuWidget(MultiBoxWidget);
	}
	MultiBox.Get().SetStyle(&FVREditorStyle::Get(), StyleName);

	QuickRadialMenu->SetNumberOfEntries((float)NumItems);

	for (const TSharedRef<const FMultiBlock>& MultiBlock : Blocks)
	{
		const TSharedRef<const FMultiBlock>& Block = MultiBlock;
		if (Block->GetType() == EMultiBlockType::MenuEntry)
		{
			TSharedRef<SWidget> BlockWidget = Block->MakeWidget(MultiBoxWidget, EMultiBlockLocation::Middle, true, nullptr)->AsWidget();
			TSharedRef<SOverlay> TestOverlay = SNew(SOverlay);

			BlockWidget = AddHoverableButton(BlockWidget, ButtonTypeOverride, TestOverlay);
			BlockWidget = SetButtonFormatting(BlockWidget, 50.0f);
			TSharedRef<SBox> RadialMenuElement = SNew(SBox);
			RadialMenuElement->SetContent(
				SNew(SDPIScaler)
				.DPIScale(3)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(5, 5, 5, 5)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						BlockWidget
					]
				]
			);

			AVREditorFloatingUI::EDockedTo RadialMenuInteractor = AVREditorFloatingUI::EDockedTo::LeftArm;
			const FIntPoint DefaultResolution(VREd::DefaultRadialElementResolutionX->GetInt(), VREd::DefaultRadialElementResolutionY->GetInt());
			QuickRadialMenu->SetSlateWidget(*this, RadialMenuElement, DefaultResolution, 40.0f, RadialMenuInteractor);
			FVRButton ButtonToAdd = FVRButton(QuickRadialMenu->GetWidgetComponents().Last(), FVector(1.0f/25.0f));
			VRButtons.Add(ButtonToAdd);
		}
	}
}

void UVREditorUISystem::BuildRadialMenuWidget()
{

	FMultiBox::FOnMakeMultiBoxBuilderOverride VREditorMenuBuilderOverride;
	
	TSharedPtr<FUICommandList> CommandList = MakeShareable(new FUICommandList());
	FMenuBuilder MenuBuilder(false, CommandList, IVREditorModule::Get().GetRadialMenuExtender());
	float RadiusOverride = 1.0f;
	if (RadialMenuHandler != nullptr)
	{
		RadialMenuHandler->BuildRadialMenuCommands(MenuBuilder, CommandList, VRMode, RadiusOverride);
	}
	VREditorMenuBuilderOverride.BindUObject(this, &UVREditorUISystem::MakeRadialBoxMenu, RadiusOverride, FName(TEXT("SMenuEntryButton")));

	// Create the menu widget
	MenuBuilder.MakeWidget(&VREditorMenuBuilderOverride);
}

void UVREditorUISystem::BuildNumPadWidget()
{

	FMultiBox::FOnMakeMultiBoxBuilderOverride VREditorMenuBuilderOverride;
	VREditorMenuBuilderOverride.BindUObject(this, &UVREditorUISystem::MakeRadialBoxMenu, 0.5f, FName(TEXT("SButton")));
	const TSharedRef<FExtender> MenuExtender = MakeShareable(new FExtender());
	TSharedPtr<FUICommandList> CommandList = MakeShareable(new FUICommandList());
	FMenuBarBuilder MenuBuilder(CommandList, MenuExtender);

	// First menu entry is at 90 degrees 

	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumThree", "3"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(3))
		)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumFour", "4"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(4))
		)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumFive", "5"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(5))
		)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumSix", "6"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(6))
		)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumSeven", "7"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(7))
			)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumEight", "8"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(8))
			)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NumNine", "9"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt(9))
		)
		);
	MenuBuilder.AddMenuEntry(
		LOCTEXT( "Hyphen", "-" ),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic( &FVREditorActionCallbacks::SimulateCharacterEntry, FString( TEXT( "-" ) ) )
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT( "Decimal", "." ),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic( &FVREditorActionCallbacks::SimulateCharacterEntry, FString( TEXT( "." ) ) )
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT( "NumZero", "0" ),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic( &FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt( 0 ) )
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT( "NumOne", "1" ),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic( &FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt( 1 ) )
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT( "NumTwo", "2" ),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction
		(
			FExecuteAction::CreateStatic( &FVREditorActionCallbacks::SimulateCharacterEntry, FString::FromInt( 2 ) )
		)
	);

	// Create the menu widget
	MenuBuilder.MakeWidget(&VREditorMenuBuilderOverride);

}

const TSharedRef<SWidget>& UVREditorUISystem::FindWidgetOfType(const TSharedRef<SWidget>& Content, FName WidgetType)
{
	if (Content->GetType() == WidgetType)
	{
		return Content;
	}

	FChildren* Children = Content->GetChildren();
	int32 NumChildren = Children->Num();

	for (int32 Index = 0; Index < NumChildren; ++Index)
	{
		const TSharedRef<SWidget>& Found = FindWidgetOfType((Children->GetChildAt(Index)), WidgetType);
		if (Found != SNullWidget::NullWidget)
		{
			return Found;
		}
	}
	return SNullWidget::NullWidget;
}

const bool UVREditorUISystem::FindAllWidgetsOfType(TArray<TSharedRef<SWidget>>& FoundWidgets, const TSharedRef<SWidget>& Content, FName WidgetType)
{
	bool bFoundMatch = false;
	if (Content->GetType() == WidgetType)
	{
		FoundWidgets.Add(Content);
		bFoundMatch = true;
	}

	FChildren* Children = Content->GetChildren();
	int32 NumChildren = Children->Num();

	for (int32 Index = 0; Index < NumChildren; ++Index)
	{
		bFoundMatch = FindAllWidgetsOfType(FoundWidgets, (Children->GetChildAt(Index)), WidgetType);
	}

	return bFoundMatch;
}



void UVREditorUISystem::OnHoverBeginEffect(UVREditorWidgetComponent* Button)
{
	FVRButton* ButtonToAnimate = VRButtons.FindByPredicate([&Button](const FVRButton& ButtonData)
	{
		return Button == ButtonData.ButtonWidget;
	});

	if (ButtonToAnimate)
	{
		//Set the newly hovered button's animation state to Forward
		ButtonToAnimate->AnimationDirection = EVREditorAnimationState::Forward;
	}
}

void UVREditorUISystem::OnHoverEndEffect(UVREditorWidgetComponent* Button)
{
	FVRButton* ButtonToAnimate = VRButtons.FindByPredicate([&Button](const FVRButton& ButtonData)
	{
		return Button == ButtonData.ButtonWidget;
	});

	if (ButtonToAnimate)
	{
		// Set the unhovered button's animation state to Backward
		ButtonToAnimate->AnimationDirection = EVREditorAnimationState::Backward;
	}

}

void UVREditorUISystem::SequencerOpenendFromRadialMenu(const bool bInOpenedFromRadialMenu /*= true*/)
{
	bSequencerOpenedFromRadialMenu = bInOpenedFromRadialMenu;
}

bool UVREditorUISystem::CanScalePanel() const
{
	return bPanelCanScale || (DraggingUI != nullptr && LaserInteractor != nullptr && InteractorDraggingUI != nullptr && LaserInteractor == InteractorDraggingUI);
}

UVREditorInteractor* UVREditorUISystem::GetUIInteractor()
{
	return UIInteractor;
}

AVREditorFloatingUI* UVREditorUISystem::GetPanel(const VREditorPanelID& InPanelID) const
{
	AVREditorFloatingUI* Result = nullptr;
	if (FloatingUIs.Num() > 0)
	{
		TObjectPtr<AVREditorFloatingUI> const * FoundPanel = FloatingUIs.Find(InPanelID);
		if (FoundPanel != nullptr)
		{
			Result = *FoundPanel;
		}
	}
	return Result;
}

void UVREditorUISystem::ResetAll()
{
	RadialMenuHandler->Home();
	HideRadialMenu(false);

	FVREditorActionCallbacks::DeselectAll();

	// Close all the editor UI panels.
	const bool bShouldShow = false;
	const bool bSpawnInFront = false;
	const bool bDragFromOpen = false;
	const bool bPlaySound = false;
	for (auto& CurrentUI : FloatingUIs)
	{
		AVREditorFloatingUI* FloatingUI = CurrentUI.Value;
		if (FloatingUI != nullptr)
		{
			ShowEditorUIPanel(FloatingUI, UIInteractor, bShouldShow, bSpawnInFront, bDragFromOpen, bPlaySound);
		}
	}
	FVREditorActionCallbacks::ChangeEditorModes(FBuiltinEditorModes::EM_Default);

	// Turn off all snapping
	if (FVREditorActionCallbacks::GetTranslationSnapState() == ECheckBoxState::Checked)
	{
		FLevelEditorActionCallbacks::LocationGridSnap_Clicked();
	}
	if (FVREditorActionCallbacks::GetRotationSnapState() == ECheckBoxState::Checked)
	{
		FLevelEditorActionCallbacks::RotationGridSnap_Clicked();
	}
	if (FVREditorActionCallbacks::GetScaleSnapState() == ECheckBoxState::Checked)
	{
		FLevelEditorActionCallbacks::ScaleGridSnap_Clicked();
	}
	if (FVREditorActionCallbacks::AreAligningToActors(VRMode) == ECheckBoxState::Checked)
	{
		FVREditorActionCallbacks::ToggleAligningToActors(VRMode);
	}
}

bool UVREditorUISystem::CheckForVRDragDrop()
{
	if (LaserInteractor != nullptr)
	{
		return LaserInteractor->IsClickingOnUI();
	}
	else
	{
		return false;
	}
}

bool UVREditorUISystem::ShouldPreviewPanel()
{
	bool bDragFromOpen = true;
	if (LaserInteractor != nullptr)
	{
		// If we are clicking with the laser interactor, instantly spawn the panel
		FHitResult HitResult = LaserInteractor->GetHitResultFromLaserPointer();
		if (HitResult.HitObjectHandle.IsValid())
		{
			if (HitResult.HitObjectHandle == QuickRadialMenu)
			{
				bDragFromOpen = !LaserInteractor->IsClickingOnUI();
			}
		}
	}
	return bDragFromOpen;
}

AVREditorRadialFloatingUI* UVREditorUISystem::GetRadialMenuFloatingUI() const
{
	return QuickRadialMenu;
}

void UVREditorUISystem::SwapRadialMenu()
{
	if (bRadialMenuIsNumpad)
	{
		bRadialMenuIsNumpad = false;
	}
	else
	{
		bRadialMenuIsNumpad = true;
	}
	const bool bForceRefresh = true;
	TryToSpawnRadialMenu(UIInteractor, bForceRefresh);
}

void UVREditorUISystem::UpdateSequencerUI()
{
	AVREditorFloatingUI* SequencerPanel = GetPanel(SequencerPanelID);
	if (SequencerPanel != nullptr)
	{
		ISequencer* Sequencer = GetOwner().GetCurrentSequencer();
		if(Sequencer != nullptr)
		{
			const TSharedRef<SWidget> SequencerWidget = Sequencer->GetSequencerWidget();

			TSharedRef<SWidget> WidgetToDraw = SNew(SDPIScaler)
				.DPIScale(1.0f)
				[
					SequencerWidget
				];

			const bool bWithSceneComponent = false;
			SequencerPanel->SetSlateWidget(*this, SequencerPanelID, WidgetToDraw, FIntPoint((int32)VREd::SequencerUIResolutionX->GetFloat(), (int32)VREd::SequencerUIResolutionY->GetFloat()), VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
	
			if (bSequencerOpenedFromRadialMenu == true)
			{
				const bool bDragFromOpen = ShouldPreviewPanel();
				const bool bShouldShow = true;
				const bool bSpawnInFront = true;
				ShowEditorUIPanel(SequencerPanel, UIInteractor, bShouldShow, bSpawnInFront, bDragFromOpen);
				bSequencerOpenedFromRadialMenu = false;
			}

			if (InfoDisplayPanel != nullptr)
			{
				const TSharedRef<SWidget> SequencerTimer = Sequencer->GetTopTimeSliderWidget().ToSharedRef();
				TSharedRef<SWidget> SequencerTimerToDraw = SNew(SDPIScaler)
					.DPIScale(3.0f)
					[
						SequencerTimer	
					];
				CurrentWidgetOnInfoDisplay = SequencerTimerToDraw;
				
				InfoDisplayPanel->SetSlateWidget(SequencerTimerToDraw);

				const AVREditorFloatingUI::EDockedTo DockTo = LaserInteractor == nullptr ? AVREditorFloatingUI::EDockedTo::Nothing :
					LaserInteractor->GetControllerSide() == EControllerHand::Left ? AVREditorFloatingUI::EDockedTo::LeftHand : AVREditorFloatingUI::EDockedTo::RightHand;
				InfoDisplayPanel->SetDockedTo(DockTo);
				InfoDisplayPanel->ShowUI(true);
			}
		}
		else
		{
			// Hide the info display when finished with sequencer.
			if (InfoDisplayPanel != nullptr)
			{
				const TSharedPtr<SWidget> Widget = InfoDisplayPanel->GetSlateWidget();
				if (Widget.Get() != nullptr && Widget != SNullWidget::NullWidget && Widget == CurrentWidgetOnInfoDisplay)
				{
					InfoDisplayPanel->ShowUI(false, true, 0.0f, true);
					CurrentWidgetOnInfoDisplay.Reset();
				}
			}
		}
	}
}

void UVREditorUISystem::UpdateActorPreviewUI( TSharedRef<SWidget> InWidget, int32 Index, AActor *Actor )
{
	
	FString PanelString = ActorPreviewPrefix;
	if (Actor)
	{
		PanelString += FString::FromInt(Actor->GetUniqueID());
	}
	VREditorPanelID PreviewPanelID = FName(*PanelString);
	AVREditorFloatingUI* PreviewPanel = GetPanel(PreviewPanelID);

	// Cleanup the UI if no widget is sent for update.
	if (PreviewPanel && InWidget == SNullWidget::NullWidget)
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveActorPreviewUI", "Remove ActorPreviewUI"));
		FloatingUIs.Remove(PreviewPanelID);
		PreviewWindowInfo.Remove(PreviewPanelID);
		GetOwner().GetWorld()->DestroyActor( PreviewPanel );
		GetOwner().GetLevelViewportPossessedForVR().GetLevelViewportClient().EngineShowFlags.SetSelectionOutline( true );
		return;
	}

	const FIntPoint ViewfinderSize = FIntPoint((int32)VREd::DefaultCameraUIResolutionX->GetFloat(), (int32)VREd::DefaultCameraUIResolutionY->GetFloat());

	// If a widget is sent but no panel exists, create one.
	if (Actor && PreviewPanel == nullptr)
	{
		const FName TabIdentifier = NAME_None;	// No tab for us!

		UWorld* World = GetOwner().GetWorld();
		check( World != nullptr );

		// if currently in PIE, non-PIE actors should be spawned in LastEditorWorld if it exists.
		if (!GEditor->bIsSimulatingInEditor && GEditor->PlayWorld != nullptr && GEditor->PlayWorld == World)
		{
			UWorld* LastEditorWorld = GetOwner().GetLastEditorWorld();
			if (LastEditorWorld != nullptr)
			{
				World = LastEditorWorld;
			}
		}

		FActorSpawnParameters ActorSpawnParameters;
		ActorSpawnParameters.Name = MakeUniqueObjectName(World->GetCurrentLevel(), AVREditorFloatingCameraUI::StaticClass(), TEXT( "ActorPreviewUI" ) );
		ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		ActorSpawnParameters.bHideFromSceneOutliner = true;
		ActorSpawnParameters.ObjectFlags = RF_Transactional;

		PreviewPanel = World->SpawnActor<AVREditorFloatingCameraUI>( ActorSpawnParameters );
		PreviewPanel->SetActorLabel( "ActorPreviewUI" );

		FloatingUIs.Add(PreviewPanelID, PreviewPanel);
		PreviewWindowInfo.Add( PreviewPanelID, Actor );
		Cast<AVREditorFloatingCameraUI>( PreviewPanel )->SetLinkedActor( Actor );

		PreviewPanel->SetSlateWidget( *this, PreviewPanelID, SNullWidget::NullWidget, ViewfinderSize, VREd::CameraPreviewUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing );
		PreviewPanel->ShowUI( true );
	}
	
	// Once a panel exists:
	if (PreviewPanel != nullptr)
	{
		TSharedRef<SWidget> WidgetToDraw =
			SNew( SDPIScaler )
			.DPIScale( VREd::EditorUIScale->GetFloat() )
			[
				InWidget
			]
		;

		PreviewPanel->SetSlateWidget( *this, PreviewPanelID, WidgetToDraw, ViewfinderSize, VREd::CameraPreviewUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing );

		// If we are not using the global preview, spawn the UI and dock it to the camera
		const bool bDragFromOpen = false;
		const bool bShouldShow = true;
		const bool bSpawnInFront = true;
		ShowEditorUIPanel( PreviewPanel, LaserInteractor, bShouldShow, bSpawnInFront, bDragFromOpen );

		PreviewPanel->SetDockedTo( AVREditorFloatingUI::EDockedTo::Custom );
		// We don't want the camera outline to show through the UI
		GetOwner().GetLevelViewportPossessedForVR().GetLevelViewportClient().EngineShowFlags.SetSelectionOutline( false );
	}
}

void UVREditorUISystem::UpdateDetachedActorPreviewUI(TSharedRef<SWidget> InWidget, int32 Index)
{
	FString PanelString = ActorPreviewPrefix + FString::FromInt(Index);
	const VREditorPanelID PreviewPanelID = FName(*PanelString);
	AVREditorFloatingUI* PreviewPanel = GetPanel(PreviewPanelID);

	// Cleanup the UI if no widget is sent for update.
	if (PreviewPanel && InWidget == SNullWidget::NullWidget)
	{
		FloatingUIs.Remove(PreviewPanelID);
		PreviewWindowInfo.Remove(PreviewPanelID);
		VRMode->DestroyTransientActor(PreviewPanel);
		return;
	}
	FIntPoint ViewfinderSize = FIntPoint((int32)VREd::DefaultCameraUIResolutionX->GetFloat(), (int32)VREd::DefaultCameraUIResolutionY->GetFloat());

	// If a widget is sent but no panel exists, create one.
	if (PreviewPanel == nullptr)
	{
		const FName TabIdentifier = NAME_None;	// No tab for us!
		const bool bWithSceneComponent = false;
		PreviewPanel = GetOwner().SpawnTransientSceneActor<AVREditorDockableCameraWindow>(TEXT("ActorDetachablePreviewUI"), bWithSceneComponent);
		PreviewPanel->CreationContext.bNoCloseButton = true;
		FloatingUIs.Add(PreviewPanelID, PreviewPanel);
		AActor* Actor = GEditor->GetSelectedActors()->GetBottom<AActor>();
		if (Actor)
		{
			PreviewWindowInfo.Add(PreviewPanelID, Actor);
		}
		PreviewPanel->SetSlateWidget(*this, PreviewPanelID, SNullWidget::NullWidget, ViewfinderSize, VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
		PreviewPanel->ShowUI(true);
	}

	// Once a panel exists:
	if (PreviewPanel != nullptr)
	{
		TSharedRef<SWidget> WidgetToDraw =
			SNew(SDPIScaler)
			.DPIScale(3.0f)
			[
				InWidget
			]
		;
	
		PreviewPanel->SetSlateWidget(*this, PreviewPanelID, WidgetToDraw, ViewfinderSize, VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
		const bool bDragFromOpen = false;
		const bool bShouldShow = true;
		const bool bSpawnInFront = true;
		ShowEditorUIPanel(PreviewPanel, UIInteractor, bShouldShow, bSpawnInFront, bDragFromOpen);
		// Make sure the UIs are centered around the direction your head is looking (yaw only!)
		const FVector RoomSpaceHeadLocation = VRMode->GetRoomSpaceHeadTransform().GetLocation() / VRMode->GetWorldScaleFactor();
		FRotator RoomSpaceHeadYawRotator = VRMode->GetRoomSpaceHeadTransform().GetRotation().Rotator();
		RoomSpaceHeadYawRotator.Pitch = 0.0f;
		RoomSpaceHeadYawRotator.Roll = 0.0f;

		FTransform NewTransform = VREd::DefaultActorPreviewTransform;
		// Offset slightly for each new window
		FVector Translation = NewTransform.GetTranslation();
		Translation.X -= Index;
		Translation.Y += Index;
		NewTransform.SetTranslation(Translation);
		if (GetDefault<UVRModeSettings>()->InteractorHand == EInteractorHand::Left)
		{
			NewTransform.Mirror(EAxis::Y, EAxis::Y);
		}
		NewTransform *= FTransform(RoomSpaceHeadYawRotator.Quaternion(), FVector::ZeroVector);
		PreviewPanel->SetLocalRotation(NewTransform.GetRotation().Rotator());
		PreviewPanel->SetRelativeOffset(RoomSpaceHeadLocation + NewTransform.GetTranslation());
	}
}

void UVREditorUISystem::UpdateExternalUMGUI(const FVREditorFloatingUICreationContext& CreationContext) 
{
	const VREditorPanelID PreviewPanelID = CreationContext.PanelID;
	AVREditorFloatingUI* ExternalPanel = GetPanel(PreviewPanelID);

	// Cleanup the UI if no widget is sent for update.
	if (ExternalPanel && CreationContext.WidgetClass == nullptr)
	{
		FloatingUIs.Remove(PreviewPanelID);
		VRMode->DestroyTransientActor(ExternalPanel);
		return;
	}

	// If a widget is sent but no panel exists, create one.
	if (ExternalPanel == nullptr)
	{
		const FName TabIdentifier = NAME_None;	// No tab for us!
		const bool bWithSceneComponent = false;
  		ExternalPanel = GetOwner().SpawnTransientSceneActor<AVREditorDockableWindow>(TEXT("ActorPreviewUMGUI"), bWithSceneComponent);	
		
		// AVREditorDockableWindow does most of its setup in PostActorCreated(), but the CreationContext hasn't been set yet. 
		// We let it set its defaults, then override the custom settings here.
		ExternalPanel->CreationContext = CreationContext;				
		if (CreationContext.PanelMesh != nullptr)
		{
			ExternalPanel->SetWindowMesh(CreationContext.PanelMesh);
		}
				
		FloatingUIs.Add(PreviewPanelID, ExternalPanel);
		ExternalPanel->SetUMGWidget(*this, PreviewPanelID, nullptr, FIntPoint((int32)VREd::DefaultEditorUIResolutionX->GetFloat(), (int32)VREd::DefaultEditorUIResolutionY->GetFloat()), VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
		ExternalPanel->ShowUI(true);
	}

	// Once a panel exists:
	if (ExternalPanel != nullptr)
	{
		UWorld* World = GetOwner().GetWorld();
		if (World != NULL)
		{
			FIntPoint UISize = FIntPoint((int32)VREd::DefaultEditorUIResolutionX->GetFloat(), (int32)VREd::DefaultEditorUIResolutionY->GetFloat());
			if (CreationContext.PanelSize != FVector2D::ZeroVector) 
			{
				UISize = FIntPoint((int32)CreationContext.PanelSize.X, (int32)CreationContext.PanelSize.Y);
			}
			
			// @todo: Currently passes CreationContext params into SetUMGWidget(). AVREditorFloatingUI has the context and should use it instead (deprecate the old individual vars)
			const float InEditorUISize = CreationContext.EditorUISize == 0 ? VREd::EditorUISize->GetFloat() : CreationContext.EditorUISize;			
			ExternalPanel->SetUMGWidget(*this, CreationContext.PanelID, CreationContext.WidgetClass, UISize, InEditorUISize, AVREditorFloatingUI::EDockedTo::Nothing);
			
			// Don't consider dragging if we have a fixed offset and the controller doesn't determine window position
			const bool bOpenAtControllerPosition = CreationContext.PanelSpawnOffset.Equals(FTransform::Identity);			
			const bool bDragFromOpen = bOpenAtControllerPosition ? ShouldPreviewPanel() : false;
			
			const bool bShouldShow = true;
			const bool bSpawnInFront = true;
			ShowEditorUIPanel(ExternalPanel, UIInteractor, bShouldShow, bSpawnInFront, bDragFromOpen);
		}
	}
}

void UVREditorUISystem::UpdateExternalSlateUI(TSharedRef<SWidget> InWidget, FName Name, FVector2D InSize)
{
	const VREditorPanelID PreviewPanelID = Name;
	AVREditorFloatingUI* ExternalPanel = GetPanel(PreviewPanelID);

	// Cleanup the UI if no widget is sent for update.
	if (ExternalPanel && InWidget == SNullWidget::NullWidget)
	{
		FloatingUIs.Remove(PreviewPanelID);
		VRMode->DestroyTransientActor(ExternalPanel);
		return;
	}

	// If a widget is sent but no panel exists, create one.
	if (ExternalPanel == nullptr)
	{
		const FName TabIdentifier = NAME_None;	// No tab for us!
		const bool bWithSceneComponent = false;
		ExternalPanel = GetOwner().SpawnTransientSceneActor<AVREditorDockableWindow>(TEXT("ActorPreviewUI"), bWithSceneComponent);
		FloatingUIs.Add(PreviewPanelID, ExternalPanel);
		ExternalPanel->SetSlateWidget(*this, PreviewPanelID, SNullWidget::NullWidget, FIntPoint((int32)VREd::DefaultEditorUIResolutionX->GetFloat(), (int32)VREd::DefaultEditorUIResolutionY->GetFloat()), VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
		ExternalPanel->ShowUI(true);
	}

	// Once a panel exists:
	if (ExternalPanel != nullptr)
	{
		UWorld* World = GetOwner().GetWorld();
		if (World != NULL)
		{
			FIntPoint UISize = FIntPoint((int32)VREd::DefaultEditorUIResolutionX->GetFloat(), (int32)VREd::DefaultEditorUIResolutionY->GetFloat());
			if (InSize != FVector2D::ZeroVector)
			{
				UISize = FIntPoint((int32)InSize.X, (int32)InSize.Y);
			}
			ExternalPanel->SetSlateWidget(*this, PreviewPanelID, InWidget, UISize, VREd::EditorUISize->GetFloat(), AVREditorFloatingUI::EDockedTo::Nothing);
			const bool bDragFromOpen = ShouldPreviewPanel();
			const bool bShouldShow = true;
			const bool bSpawnInFront = true;
			ShowEditorUIPanel(ExternalPanel, UIInteractor, bShouldShow, bSpawnInFront, bDragFromOpen);
		}
	}
}

void UVREditorUISystem::TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState)
{
	check(NewWorld != nullptr);
	
	if (TransitionState == EEditorWorldExtensionTransitionState::TransitionAll ||
		TransitionState == EEditorWorldExtensionTransitionState::TransitionNonPIEOnly)
	{
		for (auto& CurrentUI : FloatingUIs)
		{
			AVREditorFloatingUI* FloatingUI = CurrentUI.Value;
			if (FloatingUI != nullptr)
			{
				UUserWidget* UserWidget = FloatingUI->GetUserWidget();
				if (UserWidget != nullptr)
				{
					// Only reparent the UserWidget if it was parented to a level to begin with.  It may have been parented to an actor or
					// some other object that doesn't require us to rename anything
					ULevel* ExistingWidgetOuterLevel = Cast<ULevel>(UserWidget->GetOuter());
					if (ExistingWidgetOuterLevel != nullptr && ExistingWidgetOuterLevel != NewWorld->PersistentLevel)
					{
						UserWidget->Rename(nullptr, NewWorld->PersistentLevel);
					}
				}
			}
		}

		AVREditorFloatingUI* TabManagerUI = GetPanel(TabManagerPanelID);
		if (TabManagerUI != nullptr)
		{
			TabManagerUI->GetWidgetComponent()->UpdateWidget();
			ProxyTabManager->SetParentWindow(TabManagerUI->GetWidgetComponent()->GetSlateWindow().ToSharedRef());
		}

		TryToSpawnRadialMenu(GetUIInteractor(), true, false);
	}
}

void UVREditorUISystem::ToggledDebugMode(bool bDebugModeEnabled)
{
	const bool bShowAllFloatingUIs = bDebugModeEnabled;

	for (auto& CurrentUI : FloatingUIs)
	{
		CurrentUI.Value->ShowUI(bShowAllFloatingUIs, false);
	}

	if (QuickRadialMenu != nullptr)
	{
		QuickRadialMenu->ShowUI(bShowAllFloatingUIs, false);
	}
}

void UVREditorUISystem::OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason)
{
	AVREditorFloatingUI* AssetEditorPanel = GetPanel(TabManagerPanelID);
	if(AssetEditorPanel)
	{
		AssetEditorPanel->SetSlateWidget(SNullWidget::NullWidget);
		AssetEditorPanel->ShowUI(false);
	}
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
