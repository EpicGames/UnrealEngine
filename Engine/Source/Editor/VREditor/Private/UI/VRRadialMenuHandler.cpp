// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/VRRadialMenuHandler.h"

#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"

#include "Framework/MultiBox/MultiBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Layout/WidgetPath.h"

#include "Engine/EngineTypes.h"
#include "Editor/UnrealEdEngine.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/WorldSettings.h"
#include "Sound/SoundCue.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UnrealEdGlobals.h"

#include "VREditorMode.h"
#include "UI/VREditorUISystem.h"
#include "VREditorBaseActor.h"
#include "UI/VREditorBaseUserWidget.h"
#include "UI/VREditorFloatingUI.h"
#include "UI/VREditorDockableWindow.h"
#include "ViewportInteractionTypes.h"
#include "IHeadMountedDisplay.h"
#include "ViewportWorldInteraction.h"
#include "VREditorInteractor.h"
#include "IVREditorModule.h"
#include "Settings/EditorExperimentalSettings.h"

// UI
#include "Components/WidgetComponent.h"
#include "VREditorWidgetComponent.h"
#include "VREditorStyle.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"

// Content Browser
#include "CollectionManagerTypes.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

// World Outliner
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"

// Actor Details, Modes
#include "SEditorViewport.h"

#include "ILevelEditor.h"
#include "LevelEditor.h"

// World Settings
#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#include "SLevelViewport.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/SWidget.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Framework/MultiBox/MultiBox.h"
#include "LevelEditorActions.h"
#include "VREditorActions.h"
#include "Framework/Commands/GenericCommands.h"
#include "ISequencer.h"
#include "Kismet2/DebuggerCommands.h"
#include "UMGStyle.h"

#include "UI/VREditorRadialFloatingUI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VRRadialMenuHandler)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "VREditor"

FText UVRRadialMenuHandler::ActionMenuLabel = LOCTEXT("DefaultActions", "Actions");

UVRRadialMenuHandler::UVRRadialMenuHandler():
	Super(),
	UIOwner(nullptr)
{

}

void UVRRadialMenuHandler::BackOutMenu()
{
	if (MenuStack.Num() > 0)
	{
		FOnRadialMenuGenerated PreviousMenu = MenuStack.Pop();
		const bool bShouldAddToStack = false;
		RegisterMenuGenerator(PreviousMenu, bShouldAddToStack);
	}
}

void UVRRadialMenuHandler::Home()
{
	MenuStack.Empty();
	const bool bShouldAddToStack = false;
	RegisterMenuGenerator(HomeMenu, bShouldAddToStack);
}

void UVRRadialMenuHandler::Init(UVREditorUISystem* InUISystem)
{
	UIOwner = InUISystem;
	HomeMenu.BindUObject(this, &UVRRadialMenuHandler::HomeMenuGenerator);
	SnapMenu.BindUObject(this, &UVRRadialMenuHandler::SnapMenuGenerator);
	GizmoMenu.BindUObject(this, &UVRRadialMenuHandler::GizmoMenuGenerator);
	EditMenu.BindUObject(this, &UVRRadialMenuHandler::EditMenuGenerator);
	SystemMenu.BindUObject(this, &UVRRadialMenuHandler::SystemMenuGenerator);

	Home();
}

void UVRRadialMenuHandler::BuildRadialMenuCommands(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride)
{
	UVRRadialMenuHandler::OnRadialMenuGenerated.ExecuteIfBound(MenuBuilder, CommandList, VRMode, RadiusOverride);



	if (UIOwner != nullptr && UIOwner->GetRadialMenuFloatingUI() != nullptr)
	{
		TSharedPtr<SWidget> HomeWidget = SNullWidget::NullWidget;
		switch (MenuStack.Num())
		{
		case 0:
		{
			HomeWidget = SNew(SImage)
				.Image(FVREditorStyle::GetBrush("VREditorStyle.Home"));
			break;
		}
		case 1:
		{
			HomeWidget = SNew(SImage)
				.Image(FVREditorStyle::GetBrush("VREditorStyle.OneLevel"));
			break;
		}
		case 2:
		{
			HomeWidget = SNew(SImage)
				.Image(FVREditorStyle::GetBrush("VREditorStyle.TwoLevel"));
			break;
		}
		default:
		{
			break;
		}
		}
		if (HomeWidget != SNullWidget::NullWidget)
		{
			UIOwner->GetRadialMenuFloatingUI()->UpdateCentralWidgetComponent(HomeWidget);
		}
	}
}

void UVRRadialMenuHandler::HomeMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride)
{
	MenuBuilder.BeginSection("Home");

	// First menu entry is at 90 degrees 
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SnapSettings", "Snapping"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.SnapMenu"),
		FUIAction
		(
			FExecuteAction::CreateUObject(this, &UVRRadialMenuHandler::RegisterMenuGenerator, SnapMenu, true),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("GizmoModes", "Gizmo"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.GizmoMenu"),
		FUIAction
		(
			FExecuteAction::CreateUObject(this, &UVRRadialMenuHandler::RegisterMenuGenerator, GizmoMenu, true),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Edit", "Edit"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.EditMenu"),
		FUIAction
		(
			FExecuteAction::CreateUObject(this, &UVRRadialMenuHandler::RegisterMenuGenerator, EditMenu, true),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("System", "System"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.SystemMenu"),
		FUIAction
		(
			FExecuteAction::CreateUObject(this, &UVRRadialMenuHandler::RegisterMenuGenerator, SystemMenu, true),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);

	MenuBuilder.EndSection();
}

void UVRRadialMenuHandler::SnapMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride)
{
	MenuBuilder.BeginSection("Snap");

	FVREditorActionCallbacks::UpdateSelectingCandidateActorsText(VRMode);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleTranslationSnap", "Translate Snap"),
		LOCTEXT("ToggleTranslationSnapTooltip", "Toggle Translation Snap"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.TranslateSnap"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::LocationGridSnap_Clicked),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::GetTranslationSnapState)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	TAttribute<FText> DynamicTranslationSizeLabel;
	DynamicTranslationSizeLabel.BindStatic(&FVREditorActionCallbacks::GetTranslationSnapSizeText);
	MenuBuilder.AddMenuEntry(
		DynamicTranslationSizeLabel,
		LOCTEXT("ToggleTranslationSnapSizeTooltip", "Toggle Translation Snap Size"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.GridNum"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::OnTranslationSnapSizeButtonClicked),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleRotationSnap", "Rotate Snap"),
		LOCTEXT("ToggleRotationSnapTooltip", "Toggle Rotation Snap"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.RotateSnap"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::RotationGridSnap_Clicked),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::GetRotationSnapState)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	TAttribute<FText> DynamicRotationSizeLabel;
	DynamicRotationSizeLabel.BindStatic(&FVREditorActionCallbacks::GetRotationSnapSizeText);
	MenuBuilder.AddMenuEntry(
		DynamicRotationSizeLabel,
		LOCTEXT("ToggleRotationSnapSizeTooltip", "Toggle Rotation Snap Size"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.AngleNum"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::OnRotationSnapSizeButtonClicked),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ToggleScaleSnap", "Scale Snap"),
		LOCTEXT("ToggleScaleSnapTooltip", "Toggle Scale Snap"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.ScaleSnap"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ScaleGridSnap_Clicked),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::GetScaleSnapState)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	TAttribute<FText> DynamicScaleSizeLabel;
	DynamicScaleSizeLabel.BindStatic(&FVREditorActionCallbacks::GetScaleSnapSizeText);
	MenuBuilder.AddMenuEntry(
		DynamicScaleSizeLabel,
		LOCTEXT("ToggleScaleSnapSizeTooltip", "Toggle Scale Snap Size"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.ScaleNum"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::OnScaleSnapSizeButtonClicked),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SmartSnapping", "Smart Snapping"),
		LOCTEXT("AlignToActorsTooltip", "Align to Actors as you transform an object"),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.AlignActors"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::ToggleAligningToActors, VRMode),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::AreAligningToActors, VRMode)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	TAttribute<FText> DynamicAlignSelectionLabel;
	DynamicAlignSelectionLabel.BindStatic(&FVREditorActionCallbacks::GetSelectingCandidateActorsText);
	MenuBuilder.AddMenuEntry(
		DynamicAlignSelectionLabel,
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.SetTargets"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::ToggleSelectingCandidateActors, VRMode),
			FCanExecuteAction::CreateStatic(&FVREditorActionCallbacks::CanSelectCandidateActors, VRMode)
		),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton
	);

	MenuBuilder.EndSection();
}

void UVRRadialMenuHandler::GizmoMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride)
{
	MenuBuilder.BeginSection("Gizmo");

	MenuBuilder.AddMenuEntry(
		LOCTEXT("LocalSpace", "Local Space"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.LocalSpace"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SetCoordinateSystem, VRMode, ECoordSystem::COORD_Local),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::IsActiveCoordinateSystem, VRMode, ECoordSystem::COORD_Local)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("WorldSpace", "World Space"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.WorldSpace"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SetCoordinateSystem, VRMode, ECoordSystem::COORD_World),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::IsActiveCoordinateSystem, VRMode, ECoordSystem::COORD_World)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Universal", "Universal"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Universal"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SetGizmoMode, VRMode, EGizmoHandleTypes::All),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::IsActiveGizmoMode, VRMode, EGizmoHandleTypes::All)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Translate", "Translate"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Translate"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SetGizmoMode, VRMode, EGizmoHandleTypes::Translate),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::IsActiveGizmoMode, VRMode, EGizmoHandleTypes::Translate)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Rotate", "Rotate"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Rotate"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SetGizmoMode, VRMode, EGizmoHandleTypes::Rotate),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::IsActiveGizmoMode, VRMode, EGizmoHandleTypes::Rotate)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Scale", "Scale"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Scale"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::SetGizmoMode, VRMode, EGizmoHandleTypes::Scale),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction),
			FGetActionCheckState::CreateStatic(&FVREditorActionCallbacks::IsActiveGizmoMode, VRMode, EGizmoHandleTypes::Scale)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.EndSection();
}

void UVRRadialMenuHandler::EditMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride)
{
	MenuBuilder.BeginSection("Edit");

	// First menu entry is at 90 degrees 
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeselectAll", "Deselect All"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.DeselectAll"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::DeselectAll),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Delete", "Delete"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Delete"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ExecuteExecCommand, FString(TEXT("DELETE"))),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Delete_CanExecute)
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Cut", "Cut"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Cut"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ExecuteExecCommand, FString(TEXT("EDIT CUT"))),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Cut_CanExecute)
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Copy", "Copy"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Copy"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ExecuteExecCommand, FString(TEXT("EDIT COPY"))),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Copy_CanExecute)
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Duplicate", "Duplicate Selected"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Duplicate"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ExecuteExecCommand, FString(TEXT("DUPLICATE"))),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Duplicate_CanExecute)
		)
	);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Paste", "Paste"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.Paste"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ExecuteExecCommand, FString(TEXT("EDIT PASTE"))),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Paste_CanExecute)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SnapToFloor", "Snap To Floor"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.SnapToFloor"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::OnSnapActorsToGroundClicked, VRMode),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::Copy_CanExecute)
		)
	);

	MenuBuilder.EndSection();
}
void UVRRadialMenuHandler::SystemMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride)
{
	MenuBuilder.BeginSection("System");

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Exit", "Exit"),
		FText(),
		FSlateIcon(FVREditorStyle::GetStyleSetName(), "VREditorStyle.ExitVRMode"),
		FUIAction
		(
			FExecuteAction::CreateStatic(&FVREditorActionCallbacks::ExitVRMode, VRMode),
			FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::DefaultCanExecuteAction)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.EndSection();
}

void UVRRadialMenuHandler::RegisterMenuGenerator(const FOnRadialMenuGenerated NewMenuGenerator, const bool bShouldAddToStack)
{
	if (bShouldAddToStack)
	{
		FOnRadialMenuGenerated LastMenuGenerated = GetCurrentMenuGenerator();
		if (LastMenuGenerated.IsBound())
		{
			MenuStack.Push(LastMenuGenerated);
		}
	}

	OnRadialMenuGenerated = NewMenuGenerator;
	UVREditorInteractor* RadialMenuInteractor = UIOwner->GetOwner().GetHandInteractor(EControllerHand::Right);
	if (UIOwner->IsShowingRadialMenu(RadialMenuInteractor) && NewMenuGenerator.IsBound())
	{
		const bool bForceRefresh = true;
		UIOwner->TryToSpawnRadialMenu(RadialMenuInteractor, bForceRefresh);
	}
	else
	{
		RadialMenuInteractor = UIOwner->GetOwner().GetHandInteractor(EControllerHand::Left);
		if (UIOwner->IsShowingRadialMenu(RadialMenuInteractor) && NewMenuGenerator.IsBound())
		{
			const bool bForceRefresh = true;
			UIOwner->TryToSpawnRadialMenu(RadialMenuInteractor, bForceRefresh);
		}
	}
}


#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS