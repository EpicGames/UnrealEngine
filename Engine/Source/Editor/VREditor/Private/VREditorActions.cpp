// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorActions.h"
#include "VREditorModule.h"
#include "UI/VREditorUISystem.h"
#include "VREditorMode.h"
#include "ViewportWorldInteraction.h"
#include "VREditorInteractor.h"
#include "UI/VREditorFloatingUI.h"
#include "SLevelViewport.h"
#include "ImageUtils.h"
#include "Misc/FileHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"

#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "ISequencer.h"
#include "SequencerSettings.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModes.h"
#include "EdMode.h"
#include "Engine/Selection.h"
#include "VREditorInteractor.h"
#include "LevelEditorActions.h"
#include "UObject/UObjectIterator.h"
#include "Factories/Factory.h"
#include "Subsystems/AssetEditorSubsystem.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "VREditorActions"

namespace VREd
{
	static FAutoConsoleVariable AllowPlay(TEXT("VREd.AllowPlay"), 1, TEXT("Allow to start play."));
}

FText FVREditorActionCallbacks::GizmoCoordinateSystemText;
FText FVREditorActionCallbacks::GizmoModeText;
FText FVREditorActionCallbacks::SelectingCandidateActorsText;
FText FVREditorActionCallbacks::SequencerLoopText;

ECheckBoxState FVREditorActionCallbacks::GetTranslationSnapState()
{
	return GetDefault<ULevelEditorViewportSettings>()->GridEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::OnTranslationSnapSizeButtonClicked()
{
	const TArray<float>& GridSizes = GEditor->GetCurrentPositionGridArray();
	const int32 CurrentGridSize = GetDefault<ULevelEditorViewportSettings>()->CurrentPosGridSize;

	int32 NewGridSize = CurrentGridSize + 1;
	if (NewGridSize >= GridSizes.Num())
	{
		NewGridSize = 0;
	}

	GEditor->SetGridSize(NewGridSize);

	GetTranslationSnapSizeText();

}

FText FVREditorActionCallbacks::GetTranslationSnapSizeText()
{
	return FText::AsNumber(GEditor->GetGridSize());

}

ECheckBoxState FVREditorActionCallbacks::GetRotationSnapState()
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::OnRotationSnapSizeButtonClicked()
{
	const TArray<float>& GridSizes = GEditor->GetCurrentRotationGridArray();
	const int32 CurrentGridSize = GetDefault<ULevelEditorViewportSettings>()->CurrentRotGridSize;
	const ERotationGridMode CurrentGridMode = GetDefault<ULevelEditorViewportSettings>()->CurrentRotGridMode;

	int32 NewGridSize = CurrentGridSize + 1;
	if (NewGridSize >= GridSizes.Num())
	{
		NewGridSize = 0;
	}

	GEditor->SetRotGridSize(NewGridSize, CurrentGridMode);

}

FText FVREditorActionCallbacks::GetRotationSnapSizeText()
{
	return FText::AsNumber(GEditor->GetRotGridSize().Yaw);
}

ECheckBoxState FVREditorActionCallbacks::GetScaleSnapState()
{
	return GetDefault<ULevelEditorViewportSettings>()->SnapScaleEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::OnScaleSnapSizeButtonClicked()
{
	const ULevelEditorViewportSettings* ViewportSettings = GetMutableDefault<ULevelEditorViewportSettings>();
	const int32 NumGridSizes = ViewportSettings->ScalingGridSizes.Num();

	const int32 CurrentGridSize = GetDefault<ULevelEditorViewportSettings>()->CurrentScalingGridSize;

	int32 NewGridSize = CurrentGridSize + 1;
	if (NewGridSize >= NumGridSizes)
	{
		NewGridSize = 0;
	}

	GEditor->SetScaleGridSize(NewGridSize);


}

FText FVREditorActionCallbacks::GetScaleSnapSizeText()
{
	return FText::AsNumber(GEditor->GetScaleGridSize());
}

void FVREditorActionCallbacks::OnGizmoCoordinateSystemButtonClicked(UVREditorMode* InVRMode)
{
	InVRMode->GetWorldInteraction().CycleTransformGizmoCoordinateSpace();
	UpdateGizmoModeText(InVRMode);
	UpdateGizmoCoordinateSystemText(InVRMode);

}

void FVREditorActionCallbacks::SetCoordinateSystem(UVREditorMode* InVRMode, ECoordSystem InCoordSystem)
{
	if (!(InVRMode->GetWorldInteraction().GetTransformGizmoCoordinateSpace() == InCoordSystem))
	{
		InVRMode->GetWorldInteraction().SetTransformGizmoCoordinateSpace(InCoordSystem);
		UpdateGizmoCoordinateSystemText(InVRMode);
	}
}

ECheckBoxState FVREditorActionCallbacks::IsActiveCoordinateSystem(UVREditorMode* InVRMode, ECoordSystem InCoordSystem)
{
	return (InVRMode->GetWorldInteraction().GetTransformGizmoCoordinateSpace() == InCoordSystem) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


void FVREditorActionCallbacks::SetGizmoMode(UVREditorMode* InVRMode, EGizmoHandleTypes InGizmoMode)
{
	if (!(InVRMode->GetWorldInteraction().GetCurrentGizmoType() == InGizmoMode))
	{
		InVRMode->GetWorldInteraction().SetGizmoHandleType(InGizmoMode);
		UpdateGizmoCoordinateSystemText(InVRMode);
	}
}

EGizmoHandleTypes FVREditorActionCallbacks::GetGizmoMode(const UVREditorMode* InVRMode)
{
	return InVRMode->GetWorldInteraction().GetCurrentGizmoType();
}

ECheckBoxState FVREditorActionCallbacks::IsActiveGizmoMode(UVREditorMode* InVRMode, EGizmoHandleTypes InGizmoMode)
{
	return (InVRMode->GetWorldInteraction().GetCurrentGizmoType() == InGizmoMode) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText FVREditorActionCallbacks::GetGizmoCoordinateSystemText()
{
	return FVREditorActionCallbacks::GizmoCoordinateSystemText;
}

void FVREditorActionCallbacks::UpdateGizmoCoordinateSystemText(UVREditorMode* InVRMode)
{
	const ECoordSystem CurrentCoordSystem = InVRMode->GetWorldInteraction().GetTransformGizmoCoordinateSpace(); //@todo VREditor
	FVREditorActionCallbacks::GizmoCoordinateSystemText = (CurrentCoordSystem == COORD_World ? LOCTEXT("WorldCoordinateSystem", "World") : LOCTEXT("LocalCoordinateSystem", "Local"));
}

void FVREditorActionCallbacks::OnGizmoModeButtonClicked(UVREditorMode* InVRMode)
{
	const EGizmoHandleTypes CurrentGizmoMode = InVRMode->GetWorldInteraction().GetCurrentGizmoType();
	InVRMode->CycleTransformGizmoHandleType();
	UpdateGizmoModeText(InVRMode);
	UpdateGizmoCoordinateSystemText(InVRMode);

}

FText FVREditorActionCallbacks::GetGizmoModeText()
{
	return FVREditorActionCallbacks::GizmoModeText;
}

void FVREditorActionCallbacks::UpdateGizmoModeText(UVREditorMode* InVRMode)
{
	const EGizmoHandleTypes CurrentGizmoType = InVRMode->GetWorldInteraction().GetCurrentGizmoType();
	FText GizmoTypeText;
	switch (CurrentGizmoType)
	{
	case EGizmoHandleTypes::All:
		GizmoTypeText = LOCTEXT("AllGizmoType", "Universal");
		break;

	case EGizmoHandleTypes::Translate:
		GizmoTypeText = LOCTEXT("TranslateGizmoType", "Translate");
		break;

	case EGizmoHandleTypes::Rotate:
		GizmoTypeText = LOCTEXT("RotationGizmoType", "Rotate");
		break;

	case EGizmoHandleTypes::Scale:
		GizmoTypeText = LOCTEXT("ScaleGizmoType", "Scale");
		break;

	default:
		check(0);	// Unrecognized type
		break;
	}

	FVREditorActionCallbacks::GizmoModeText = GizmoTypeText;
}

void FVREditorActionCallbacks::OnUIToggleButtonClicked(UVREditorMode* InVRMode, VREditorPanelID PanelToToggle)
{
	InVRMode->GetUISystem().TogglePanelVisibility(PanelToToggle);
}

ECheckBoxState FVREditorActionCallbacks::GetUIToggledState(UVREditorMode* InVRMode, VREditorPanelID PanelToCheck)
{
	return InVRMode->GetUISystem().IsShowingEditorUIPanel(PanelToCheck) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::OnSnapActorsToGroundClicked(UVREditorMode* InVRMode)
{
	InVRMode->SnapSelectedActorsToGround();
}

void FVREditorActionCallbacks::SimulateCharacterEntry(const FString InChar)
{

	for (int32 CharIndex = 0; CharIndex < InChar.Len(); CharIndex++)
	{
		TCHAR CharKey = InChar[CharIndex];
		const bool bRepeat = false;
		FCharacterEvent CharacterEvent(CharKey, FModifierKeysState(), 0, bRepeat);
		FSlateApplication::Get().ProcessKeyCharEvent(CharacterEvent);
	}

}

void FVREditorActionCallbacks::SimulateBackspace()
{
	// Slate editable text fields handle backspace as a character entry
	FString BackspaceString = FString(TEXT("\b"));
	bool bRepeat = false;
	SimulateCharacterEntry(BackspaceString);
}

void FVREditorActionCallbacks::SimulateKeyDown(const FKey Key, const bool bRepeat)
{
	const uint32* KeyCodePtr;
	const uint32* CharCodePtr;
	FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

	uint32 KeyCode = KeyCodePtr ? *KeyCodePtr : 0;
	uint32 CharCode = CharCodePtr ? *CharCodePtr : 0;

	FKeyEvent KeyEvent( Key, FModifierKeysState(), 0, bRepeat, KeyCode, CharCode );
	bool DownResult = FSlateApplication::Get().ProcessKeyDownEvent( KeyEvent );

	if (CharCodePtr)
	{
		FCharacterEvent CharacterEvent( (TCHAR)CharCode, FModifierKeysState(), 0, bRepeat );
		FSlateApplication::Get().ProcessKeyCharEvent( CharacterEvent );
	}
}

void FVREditorActionCallbacks::SimulateKeyUp(const FKey Key)
{
	const uint32* KeyCodePtr;
	const uint32* CharCodePtr;
	FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

	uint32 KeyCode = KeyCodePtr ? *KeyCodePtr : 0;
	uint32 CharCode = CharCodePtr ? *CharCodePtr : 0;

	FKeyEvent KeyEvent( Key, FModifierKeysState(), 0, false, KeyCode, CharCode );
	FSlateApplication::Get().ProcessKeyUpEvent( KeyEvent );
}

void FVREditorActionCallbacks::CreateNewSequence(UVREditorMode* InVRMode)
{
	// Create a new level sequence
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

	UObject* NewAsset = nullptr;

	// Attempt to create a new asset
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* CurrentClass = *It;
		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->CanCreateNew() && Factory->ImportPriority >= 0 && Factory->SupportedClass == ULevelSequence::StaticClass())
			{
				FString NewPackageName;
				FString NewAssetName;
				// Sequences created in VR editor will have a sequential VRSequencer00X naming scheme and be stored in Game/Sequences
				AssetTools.CreateUniqueAssetName(TEXT("/Game/Cinematics/Sequences/VRSequence"), TEXT("001"), NewPackageName, NewAssetName);
				NewAsset = AssetTools.CreateAsset(NewAssetName, TEXT("/Game/Cinematics/Sequences"), ULevelSequence::StaticClass(), Factory);
				break;
			}
		}
	}

	if (!NewAsset)
	{
		return;
	}

	// Spawn an actor at the origin, and move in front of the camera and open for edit
	UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(ALevelSequenceActor::StaticClass());
	if (!ensure(ActorFactory))
	{
		return;
	}

	ALevelSequenceActor* NewActor = CastChecked<ALevelSequenceActor>(GEditor->UseActorFactory(ActorFactory, FAssetData(NewAsset), &FTransform::Identity));

	InVRMode->GetUISystem().SequencerOpenendFromRadialMenu(true);

	// Open the Sequencer window
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);
}

void FVREditorActionCallbacks::CloseSequencer(UMovieSceneSequence* OpenSequence)
{
	IAssetEditorInstance* SequencerEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(OpenSequence, false);
	if (SequencerEditor != nullptr)
	{
		SequencerEditor->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
	}
}

void FVREditorActionCallbacks::PlaySequenceAtRate(UVREditorMode* InVRMode, float Rate)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	if (CurrentSequencer != nullptr)
	{
		CurrentSequencer->SetPlaybackSpeed(Rate);
		CurrentSequencer->OnPlay(false);
	}
}

void FVREditorActionCallbacks::PauseSequencePlayback(UVREditorMode* InVRMode)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	if (CurrentSequencer != nullptr)
	{
		CurrentSequencer->SetPlaybackSpeed(1.0f);
		CurrentSequencer->Pause();
	}
}

void FVREditorActionCallbacks::PlayFromBeginning(UVREditorMode* InVRMode)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	if (CurrentSequencer != nullptr)
	{
		CurrentSequencer->SetLocalTime(0);
		CurrentSequencer->SetPlaybackSpeed(1.f);
		CurrentSequencer->OnPlay(false);
	}
}

void FVREditorActionCallbacks::ToggleLooping(UVREditorMode* InVRMode)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	
	if (CurrentSequencer != nullptr)
	{
		ESequencerLoopMode LoopMode = CurrentSequencer->GetSequencerSettings()->GetLoopMode();
		if (LoopMode == ESequencerLoopMode::SLM_NoLoop)
		{
			CurrentSequencer->GetSequencerSettings()->SetLoopMode(ESequencerLoopMode::SLM_Loop);
		}
		else if (LoopMode == ESequencerLoopMode::SLM_Loop && !CurrentSequencer->GetSelectionRange().IsEmpty())
		{
			CurrentSequencer->GetSequencerSettings()->SetLoopMode(ESequencerLoopMode::SLM_LoopSelectionRange);
		}
		else if (LoopMode == ESequencerLoopMode::SLM_LoopSelectionRange || CurrentSequencer->GetSelectionRange().IsEmpty())
		{
			CurrentSequencer->GetSequencerSettings()->SetLoopMode(ESequencerLoopMode::SLM_NoLoop);
		}
	}
}

FText FVREditorActionCallbacks::GetSequencerLoopingText()
{
	return FVREditorActionCallbacks::SequencerLoopText;
}

void FVREditorActionCallbacks::UpdateSequencerLoopingText(UVREditorMode* InVRMode)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	if (CurrentSequencer != nullptr)
	{
		const ESequencerLoopMode CurrentLoopingType = CurrentSequencer->GetSequencerSettings()->GetLoopMode();
		FText LoopText;
		switch (CurrentLoopingType)
		{
		case ESequencerLoopMode::SLM_NoLoop:
			LoopText = LOCTEXT("NoLooping", "No Looping");
			break;

		case ESequencerLoopMode::SLM_Loop:
			LoopText = LOCTEXT("LoopAll", "Loop All");
			break;

		case ESequencerLoopMode::SLM_LoopSelectionRange:
			LoopText = LOCTEXT("LoopRange", "Loop Range");
			break;

		default:
			check(0);	// Unrecognized type
			break;
		}

		FVREditorActionCallbacks::SequencerLoopText = LoopText;
	}
}

void FVREditorActionCallbacks::SetSelectionRangeStart(UVREditorMode* InVRMode)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	if (CurrentSequencer != nullptr)
	{
		CurrentSequencer->SetSelectionRangeStart(CurrentSequencer->GetLocalTime().Time);
	}
}

void FVREditorActionCallbacks::SetSelectionRangeEnd(UVREditorMode* InVRMode)
{
	ISequencer* CurrentSequencer = InVRMode->GetCurrentSequencer();
	if (CurrentSequencer != nullptr)
	{
		CurrentSequencer->SetSelectionRangeEnd(CurrentSequencer->GetLocalTime().Time);
	}
}

void FVREditorActionCallbacks::ToggleSequencerScrubbing(UVREditorMode* InVRMode, UVREditorInteractor* InController)
{
	InController->ToggleSequencerScrubbingMode();
	if (!InController->IsScrubbingSequencer())
	{
		PauseSequencePlayback(InVRMode);
	}
}

ECheckBoxState FVREditorActionCallbacks::GetSequencerScrubState(UVREditorInteractor* InController)
{
	return InController->IsScrubbingSequencer() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::ToggleAligningToActors(UVREditorMode* InVRMode)
{
	if (InVRMode->GetWorldInteraction().AreAligningToActors())
	{
		// Deselect any candidates if we have them
		if (InVRMode->GetWorldInteraction().HasCandidatesSelected())
		{
			ToggleSelectingCandidateActors(InVRMode);
		}
		GUnrealEd->Exec(InVRMode->GetWorld(), TEXT("VI.ActorSnap 0"));
	}
	else
	{
		GUnrealEd->Exec(InVRMode->GetWorld(), TEXT("VI.ActorSnap 1"));
	}
}

ECheckBoxState FVREditorActionCallbacks::AreAligningToActors(UVREditorMode* InVRMode)
{
	return InVRMode->GetWorldInteraction().AreAligningToActors() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::ToggleSelectingCandidateActors(UVREditorMode* InVRMode)
{
	InVRMode->GetWorldInteraction().SetSelectionAsCandidates();
	UpdateSelectingCandidateActorsText(InVRMode);
}

bool FVREditorActionCallbacks::CanSelectCandidateActors(UVREditorMode* InVRMode)
{
	return InVRMode->GetWorldInteraction().AreAligningToActors();
}

FText FVREditorActionCallbacks::GetSelectingCandidateActorsText()
{
	return FVREditorActionCallbacks::SelectingCandidateActorsText;
}

void FVREditorActionCallbacks::UpdateSelectingCandidateActorsText(UVREditorMode* InVRMode)
{
	if (InVRMode->GetWorldInteraction().HasCandidatesSelected())
	{
		FVREditorActionCallbacks::SelectingCandidateActorsText = LOCTEXT("ResetCandidates", "Reset Targets");
	}
	else
	{
		FVREditorActionCallbacks::SelectingCandidateActorsText = LOCTEXT("SetCandidates", "Set Targets");
	}
}

void FVREditorActionCallbacks::ChangeEditorModes(FEditorModeID InMode)
{
	// *Important* - activate the mode first since FEditorModeTools::DeactivateMode will
	// activate the default mode when the stack becomes empty, resulting in multiple active visible modes.
	GLevelEditorModeTools().ActivateMode(InMode);
}

ECheckBoxState FVREditorActionCallbacks::EditorModeActive(FEditorModeID InMode)
{
	return GLevelEditorModeTools().IsModeActive(InMode) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FVREditorActionCallbacks::DeselectAll()
{
	GEditor->SelectNone(true, true, false);
	GEditor->GetSelectedActors()->DeselectAll();
	GEditor->GetSelectedObjects()->DeselectAll();
}

void FVREditorActionCallbacks::ExitVRMode(UVREditorMode* InVRMode)
{
	InVRMode->StartExitingVRMode();
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
