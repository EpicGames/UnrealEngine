// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UI/VREditorUISystem.h"
#include "Styling/SlateTypes.h"
#include "InputCoreTypes.h"
#include "Editor.h"
#include "UnrealWidgetFwd.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


// Forward declarations 
class UVREditorMode;
enum class EGizmoHandleTypes : uint8;

/**
* Implementation of various VR editor action callback functions
*/
class FVREditorActionCallbacks
{
public:

	/** Returns the checked state of the translation snap enable/disable button */
	static ECheckBoxState GetTranslationSnapState();

	/** Rotates through the available translation snap sizes */
	static void OnTranslationSnapSizeButtonClicked();

	/** Returns the translation snap size as text to use as the button display text */
	static FText GetTranslationSnapSizeText();

	/** Returns the checked state of the rotation snap enable/disable button */
	static ECheckBoxState GetRotationSnapState();

	/** Rotates through the available rotation snap sizes */
	static void OnRotationSnapSizeButtonClicked();

	/** Returns the rotation snap size as text to use as the button display text */
	static FText GetRotationSnapSizeText();

	/** Returns the checked state of the scale snap enable/disable button */
	static ECheckBoxState GetScaleSnapState();

	/** Rotates through the available scale snap sizes */
	static void OnScaleSnapSizeButtonClicked();

	/** Returns the scale snap size as text to use as the button display text */
	static FText GetScaleSnapSizeText();

	/**
	 * Toggles the gizmo coordinate system between local and world space
	 * @param InVRMode Currently active VRMode
	 */
	static void OnGizmoCoordinateSystemButtonClicked(UVREditorMode* InVRMode);

	static void SetCoordinateSystem(UVREditorMode* InVRMode, ECoordSystem InCoordSystem);

	static ECheckBoxState IsActiveCoordinateSystem(UVREditorMode* InVRMode, ECoordSystem InCoordSystem);

	/** Returns the gizmo coordinate system as text to use as the button display text */
	static FText GetGizmoCoordinateSystemText();

	/**
	 * Updates the gizmo coordinate system text if the coordinate system or gizmo type is changed
	 * @param InVRMode Currently active VRMode
	 */
	static void UpdateGizmoCoordinateSystemText(UVREditorMode* InVRMode);

	/**
	 * Rotates the gizmo type through universal, translate, rotate, and scale
	 * @param InVRMode Currently active VRMode
	 */
	static void OnGizmoModeButtonClicked(UVREditorMode* InVRMode);

	static void SetGizmoMode(UVREditorMode* InVRMode, EGizmoHandleTypes InGizmoMode);

	static EGizmoHandleTypes GetGizmoMode(const UVREditorMode* InVRMode);

	static ECheckBoxState IsActiveGizmoMode(UVREditorMode* InVRMode, EGizmoHandleTypes InGizmoMode);

	/** Returns the gizmo type as text to use as the button display text */
	static FText GetGizmoModeText();

	/**
	 * Updates the gizmo coordinate type text if the coordinate system or gizmo type is changed
	 * @param InVRMode Currently active VRMode
	 */
	static void UpdateGizmoModeText(UVREditorMode* InVRMode);

	/**
	 * Toggles a VR UI panel's state between visible and invisible
	 * @param InVRMode Currently active VRMode
	 * @param PanelToToggle UI panel to change the state of
	 */
	static void OnUIToggleButtonClicked(UVREditorMode* InVRMode, VREditorPanelID PanelToToggle);

	/**
	 * Returns a VR UI panel's visibility - used for check boxes on the menu button
	 * @param InVRMode Currently active VRMode
	 * @param PanelToToggle UI panel to read the state of
	 */
	static ECheckBoxState GetUIToggledState(UVREditorMode* InVRMode, VREditorPanelID PanelToCheck);

	/**
	 * Snaps currently selected Actors to the ground
	 * @param InVRMode Currently active VRMode
	 */
	static void OnSnapActorsToGroundClicked(UVREditorMode* InVRMode);

	/**
	 * Simulates the user entering characters with a keyboard for data entry
	 * @param InChar String of characters to enter
	 */
	static void SimulateCharacterEntry(const FString InChar);

	/** Send a backspace event. Slate editable text fields handle backspace as a TCHAR(8) character entry */
	static void SimulateBackspace();

	/**
	 * Simulates the user pressing a key down
	 * @param Key Key to press
	 * @param bRepeat Whether or not to repeat
	 */
	static void SimulateKeyDown( const FKey Key, const bool bRepeat );

	/**
	 * Simulates the user releasing a key
	 * @param Key Key to release
	 */
	static void SimulateKeyUp( const FKey Key );

	/** Create a new level sequence with an auto-generated name */
	static void CreateNewSequence(UVREditorMode* InVRMode);

	/** Close the sequencer instance */
	static void CloseSequencer(class UMovieSceneSequence* OpenSequence);

	/** Plays the current sequence at a specified rate */
	static void PlaySequenceAtRate(UVREditorMode* InVRMode, float Rate);

	/** Pauses sequence playback */
	static void PauseSequencePlayback(UVREditorMode* InVRMode);

	/** Plays at a rate of 1.0 from the local start of the sequence */
	static void PlayFromBeginning(UVREditorMode* InVRMode);

	/** Toggles looping the sequence */
	static void ToggleLooping(UVREditorMode* InVRMode);

	/** The type of looping for the current sequence*/
	static FText GetSequencerLoopingText();

	/** Updates the sequencer toggling display */
	static void UpdateSequencerLoopingText(UVREditorMode * InVRMode);

	/** Sets the current sequence's selection range start to the current time */
	static void SetSelectionRangeStart(UVREditorMode* InVRMode);

	/** Sets the current sequence's selection range end to the current time */
	static void SetSelectionRangeEnd(UVREditorMode* InVRMode);

	static void ToggleSequencerScrubbing(UVREditorMode* InVRMode, UVREditorInteractor* InController);

	static ECheckBoxState GetSequencerScrubState(UVREditorInteractor* InController);

	/** Toggles whether or not the world interaction method should align transformables to actors in the scene*/
	static void ToggleAligningToActors(UVREditorMode* InVRMode);

	/** True if the world interaction method is currently aligning transformables to actors in the scene*/
	static ECheckBoxState AreAligningToActors(UVREditorMode* InVRMode);

	/** Either selects or deselects alignment candidates */
	static void ToggleSelectingCandidateActors(UVREditorMode* InVRMode);

	/** Can only select candidate actors if actor alignment is active */
	static bool CanSelectCandidateActors(UVREditorMode* InVRMode);

	/** Dynamic label for the alignment candidate button */
	static FText GetSelectingCandidateActorsText();

	/** Updates the alignment candidate label based on the current aligning state */
	static void UpdateSelectingCandidateActorsText(UVREditorMode* InVRMode);

	/** Changes the editor mode to the given ID */
	static void ChangeEditorModes(FEditorModeID InMode);

	/** Checks whether the editor mode for the given ID is active*/
	static ECheckBoxState EditorModeActive(FEditorModeID InMode);

	/** Deselects everything currently selected */
	static void DeselectAll();

	/** Exit the VR mode */
	static void ExitVRMode(UVREditorMode* InVRMode);

public:
	
	static FText GizmoCoordinateSystemText;

	static FText GizmoModeText;

	static FText SelectingCandidateActorsText;

	static FText SequencerLoopText;

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
