// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorAvatarActor.h"
#include "Materials/MaterialInterface.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/Material.h"
#include "Engine/Font.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/PostProcessComponent.h"
#include "Components/TextRenderComponent.h"
#include "VREditorMode.h"
#include "ViewportInteractionTypes.h"
#include "VREditorInteractor.h"
#include "ViewportWorldInteraction.h"
#include "Engine/Engine.h"
#include "UObject/Package.h"
#include "EngineGlobals.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "IXRCamera.h"
#include "VREditorAssetContainer.h"
#include "VRModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorAvatarActor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable GridMovementTolerance( TEXT( "VREd.GridMovementTolerance" ), 0.1f, TEXT( "Tolerance for movement when the grid must disappear" ) );
	static FAutoConsoleVariable GridScaleMultiplier( TEXT( "VREd.GridScaleMultiplier" ), 35.0f, TEXT( "Scale of the grid" ) );
	static FAutoConsoleVariable GridFadeMultiplier( TEXT( "VREd.GridFadeMultiplier" ), 3.0f, TEXT( "Grid fade in/out speed, in 'fades per second'" ) );
	static FAutoConsoleVariable GridFadeStartVelocity( TEXT( "VREd.GridFadeStartVelocity" ), 10.f, TEXT( "Grid fade duration" ) );
	static FAutoConsoleVariable GridMaxOpacity( TEXT( "VREd.GridMaxFade" ), 0.8f, TEXT( "Grid maximum opacity" ) );
	static FAutoConsoleVariable GridHeightOffset( TEXT( "VREd.GridHeightOffset" ), 0.0f, TEXT( "Height offset for the world movement grid.  Useful when tracking space is not properly calibrated" ) );

	static FAutoConsoleVariable WorldMovementFogOpacity( TEXT( "VREd.WorldMovementFogOpacity" ), 0.8f, TEXT( "How opaque the fog should be at the 'end distance' (0.0 - 1.0)" ) );
	static FAutoConsoleVariable WorldMovementFogStartDistance( TEXT( "VREd.WorldMovementFogStartDistance" ), 300.0f, TEXT( "How far away fog will start rendering while in world movement mode" ) );
	static FAutoConsoleVariable WorldMovementFogEndDistance( TEXT( "VREd.WorldMovementFogEndDistance" ), 800.0f, TEXT( "How far away fog will finish rendering while in world movement mode" ) );
	static FAutoConsoleVariable WorldMovementFogSkyboxDistance( TEXT( "VREd.WorldMovementFogSkyboxDistance" ), 20000.0f, TEXT( "Anything further than this distance will be completed fogged and not visible" ) );

	static FAutoConsoleVariable ScaleProgressBarLength( TEXT( "VREd.ScaleProgressBarLength" ), 50.0f, TEXT( "Length of the progressbar that appears when scaling" ) );
	static FAutoConsoleVariable ScaleProgressBarRadius( TEXT( "VREd.ScaleProgressBarRadius" ), 1.0f, TEXT( "Radius of the progressbar that appears when scaling" ) );
}

AVREditorAvatarActor::AVREditorAvatarActor() :
	Super(),
	HeadMeshComponent(nullptr),
	WorldMovementGridMeshComponent(nullptr),
	WorldMovementGridMID(nullptr),
	WorldMovementGridOpacity( 0.0f ),
	bIsDrawingWorldMovementPostProcess( false ),
	WorldMovementPostProcessMaterial(nullptr),
	ScaleProgressMeshComponent(nullptr),
	CurrentScaleProgressMeshComponent(nullptr),
	UserScaleIndicatorText(nullptr),
	FixedUserScaleMID(nullptr),
	TranslucentFixedUserScaleMID(nullptr),
	CurrentUserScaleMID(nullptr),
	TranslucentCurrentUserScaleMID(nullptr),
	PostProcessComponent(nullptr),
	VRMode( nullptr )
{
	if (UNLIKELY(IsRunningDedicatedServer()) || HasAnyFlags(RF_ClassDefaultObject))   // @todo vreditor: Hack to avoid loading font assets in the cooker on Linux
	{
		return;
	}

	// Set root component 
	{
		USceneComponent* SceneRootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));
		AddOwnedComponent(SceneRootComponent);
		SetRootComponent(SceneRootComponent);
	}
}

void AVREditorAvatarActor::Init( UVREditorMode* InVRMode  )
{
	VRMode = InVRMode;

	// Setup the asset container.
	const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();

	// Give us a head mesh
	{
		HeadMeshComponent = NewObject<UStaticMeshComponent>(this);
		AddOwnedComponent( HeadMeshComponent );
		HeadMeshComponent->SetupAttachment( RootComponent );
		HeadMeshComponent->RegisterComponent();

		// @todo vreditor: This needs to adapt based on the device you're using
		UStaticMesh* HeadMesh = AssetContainer.GenericHMDMesh;
		check( HeadMesh != nullptr );

		HeadMeshComponent->SetStaticMesh( HeadMesh );
		HeadMeshComponent->SetMobility( EComponentMobility::Movable );
		HeadMeshComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );
		HeadMeshComponent->bSelectable = false;
	}

	// World movement grid mesh
	{
		WorldMovementGridMeshComponent = NewObject<UStaticMeshComponent>(this);
		AddOwnedComponent( WorldMovementGridMeshComponent );
		WorldMovementGridMeshComponent->SetupAttachment( RootComponent );
		WorldMovementGridMeshComponent->RegisterComponent();

		UStaticMesh* GridMesh = AssetContainer.PlaneMesh;
		check( GridMesh != nullptr );
		WorldMovementGridMeshComponent->SetStaticMesh( GridMesh );
		WorldMovementGridMeshComponent->SetMobility( EComponentMobility::Movable );
		WorldMovementGridMeshComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );
		WorldMovementGridMeshComponent->bSelectable = false;

		UMaterialInterface* GridMaterial = AssetContainer.GridMaterial;
		check( GridMaterial != nullptr );

		WorldMovementGridMID = UMaterialInstanceDynamic::Create( GridMaterial, GetTransientPackage( ) );
		check( WorldMovementGridMID != nullptr );
		WorldMovementGridMeshComponent->SetMaterial( 0, WorldMovementGridMID );

		// The grid starts off hidden
		WorldMovementGridMeshComponent->SetVisibility( false );
	}

	{
		UMaterialInterface* UserScaleIndicatorMaterial = AssetContainer.LaserPointerMaterial;
		check( UserScaleIndicatorMaterial != nullptr );

		UMaterialInterface* TranslucentUserScaleIndicatorMaterial = AssetContainer.LaserPointerTranslucentMaterial;
		check( TranslucentUserScaleIndicatorMaterial != nullptr );

		FixedUserScaleMID = UMaterialInstanceDynamic::Create( UserScaleIndicatorMaterial, GetTransientPackage() );
		check( FixedUserScaleMID != nullptr );

		TranslucentFixedUserScaleMID = UMaterialInstanceDynamic::Create( TranslucentUserScaleIndicatorMaterial, GetTransientPackage() );
		check( TranslucentFixedUserScaleMID != nullptr );
			
		CurrentUserScaleMID = UMaterialInstanceDynamic::Create( UserScaleIndicatorMaterial, GetTransientPackage() );
		check( CurrentUserScaleMID != nullptr );

		TranslucentCurrentUserScaleMID = UMaterialInstanceDynamic::Create( TranslucentUserScaleIndicatorMaterial, GetTransientPackage() );
		check( TranslucentCurrentUserScaleMID != nullptr );

		UStaticMesh* ScaleLineMesh = AssetContainer.LaserPointerMesh; //@todo VREditor: The laser pointer mesh is not a closed cylinder anymore.
		check( ScaleLineMesh != nullptr );

		// Creating the background bar progress of the scale 
		{
			ScaleProgressMeshComponent = NewObject<UStaticMeshComponent>(this);
			AddOwnedComponent( ScaleProgressMeshComponent );
			ScaleProgressMeshComponent->SetupAttachment( RootComponent );
			ScaleProgressMeshComponent->RegisterComponent();

			ScaleProgressMeshComponent->SetStaticMesh( ScaleLineMesh );
			ScaleProgressMeshComponent->SetMobility( EComponentMobility::Movable );
			ScaleProgressMeshComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );
			ScaleProgressMeshComponent->SetMaterial( 0, FixedUserScaleMID );
			ScaleProgressMeshComponent->SetMaterial( 1, TranslucentFixedUserScaleMID );
			ScaleProgressMeshComponent->bSelectable = false;

			// The user scale indicator starts invisible
			ScaleProgressMeshComponent->SetVisibility( false );
		}

		// Creating the current progress of the scale 
		{
			CurrentScaleProgressMeshComponent = NewObject<UStaticMeshComponent>(this);
			AddOwnedComponent( CurrentScaleProgressMeshComponent );
			CurrentScaleProgressMeshComponent->SetupAttachment( RootComponent );
			CurrentScaleProgressMeshComponent->RegisterComponent();

			CurrentScaleProgressMeshComponent->SetStaticMesh( ScaleLineMesh );
			CurrentScaleProgressMeshComponent->SetMobility( EComponentMobility::Movable );
			CurrentScaleProgressMeshComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );
			CurrentScaleProgressMeshComponent->SetMaterial( 0, CurrentUserScaleMID );
			CurrentScaleProgressMeshComponent->SetMaterial( 1, TranslucentCurrentUserScaleMID );
			CurrentScaleProgressMeshComponent->bSelectable = false;

			// The user scale indicator starts invisible
			CurrentScaleProgressMeshComponent->SetVisibility( false );
		}
	}

	// Creating the text for scaling
	{
		UFont* TextFont = AssetContainer.TextFont;
		check( TextFont != nullptr );

		UMaterialInterface* UserScaleIndicatorMaterial = AssetContainer.TextMaterial;
		check( UserScaleIndicatorMaterial != nullptr );

		UserScaleIndicatorText = NewObject<UTextRenderComponent>(this);
		AddOwnedComponent( UserScaleIndicatorText );
		UserScaleIndicatorText->SetupAttachment( RootComponent );
		UserScaleIndicatorText->RegisterComponent();

		UserScaleIndicatorText->SetMobility( EComponentMobility::Movable );
		UserScaleIndicatorText->SetCollisionEnabled( ECollisionEnabled::NoCollision );
		UserScaleIndicatorText->SetCollisionProfileName( UCollisionProfile::NoCollision_ProfileName );
		UserScaleIndicatorText->bSelectable = false;

		UserScaleIndicatorText->SetGenerateOverlapEvents(false);
		UserScaleIndicatorText->SetCanEverAffectNavigation( false );
		UserScaleIndicatorText->bCastDynamicShadow = false;
		UserScaleIndicatorText->bCastStaticShadow = false;
		UserScaleIndicatorText->bAffectDistanceFieldLighting = false;
		UserScaleIndicatorText->bAffectDynamicIndirectLighting = false;

		// Use a custom font.  The text will be visible up close.
		UserScaleIndicatorText->SetFont( TextFont );
		UserScaleIndicatorText->SetWorldSize( 8.0f );
		UserScaleIndicatorText->SetTextMaterial( UserScaleIndicatorMaterial );

		// Center the text horizontally
		UserScaleIndicatorText->SetHorizontalAlignment( EHTA_Center );
		UserScaleIndicatorText->SetVisibility( false );
	}

	// Load our post process material for the world movement grid
	{
		UMaterial* Material = AssetContainer.WorldMovementPostProcessMaterial;
		check( Material != nullptr );
		WorldMovementPostProcessMaterial = UMaterialInstanceDynamic::Create( Material, GetTransientPackage() );
		check( WorldMovementPostProcessMaterial != nullptr );
	}

	// Post processing
	{
		PostProcessComponent = NewObject<UPostProcessComponent>(this);
		AddOwnedComponent( PostProcessComponent );
		PostProcessComponent->SetupAttachment( this->GetRootComponent( ) );
		PostProcessComponent->RegisterComponent();

		// Unlimited size
		PostProcessComponent->bEnabled = GetDefault<UVRModeSettings>()->bShowWorldMovementPostProcess;
		PostProcessComponent->bUnbound = true;
	}

	// Set the default color for the progress bar
	{
		static FName StaticLaserColorName( "LaserColor" );
		const FLinearColor FixedProgressbarColor = VRMode->GetColor( UVREditorMode::EColors::WorldDraggingColor );
		FixedUserScaleMID->SetVectorParameterValue( StaticLaserColorName, FixedProgressbarColor );
		TranslucentFixedUserScaleMID->SetVectorParameterValue( StaticLaserColorName, FixedProgressbarColor );

		const FLinearColor CurrentProgressbarColor = VRMode->GetColor( UVREditorMode::EColors::DefaultColor );
		CurrentUserScaleMID->SetVectorParameterValue( StaticLaserColorName, CurrentProgressbarColor );
		TranslucentCurrentUserScaleMID->SetVectorParameterValue( StaticLaserColorName, CurrentProgressbarColor );
	}

	UserScaleIndicatorText->SetTextRenderColor( VRMode->GetColor( UVREditorMode::EColors::WorldDraggingColor ).ToFColor( false ) );

	// Tell the grid to stay relative to the rootcomponent
	const FAttachmentTransformRules AttachmentTransformRules = FAttachmentTransformRules( EAttachmentRule::KeepRelative, true );
	WorldMovementGridMeshComponent->AttachToComponent( RootComponent, AttachmentTransformRules );
}

void AVREditorAvatarActor::TickManually( const float DeltaTime )
{
	SetActorTransform( VRMode->GetRoomTransform() );
	const float WorldScaleFactor = VRMode->GetWorldScaleFactor();
	const FVector WorldScaleFactorVector = FVector( WorldScaleFactor );

	// Our head will lock to the HMD position
	{
		FTransform RoomSpaceTransformWithWorldToMetersScaling = VRMode->GetRoomSpaceHeadTransform( );
		RoomSpaceTransformWithWorldToMetersScaling.SetScale3D( RoomSpaceTransformWithWorldToMetersScaling.GetScale3D( ) * WorldScaleFactorVector );

		// @todo vreditor urgent: Head disabled until we can fix late frame update issue
		check( HeadMeshComponent != nullptr );
		#if 0 // && VRMode->IsActuallyUsingVR() && GEngine->XRSystem.IsValid() && GEngine->XRSystem->IsHeadTrackingAllowed() )
		{
			HeadMeshComponent->SetVisibility( true );

			// Apply late frame update to the head mesh
			HeadMeshComponent->ResetRelativeTransform( );
			const FTransform ParentToWorld = HeadMeshComponent->GetComponentToWorld( );
			auto XRCamera = GEngine->XRSystem->GetXRCamera();
			if (XRCamera.IsValid())
			{
				XRCamera->SetupLateUpdate(ParentToWorld, HeadMeshComponent, false);
			}
			HeadMeshComponent->SetRelativeTransform( RoomSpaceTransformWithWorldToMetersScaling );
		}
		#else
		{
			HeadMeshComponent->SetVisibility( false );
		}
		#endif
	}

	// Scale the grid so that it stays the same size in the tracking space
	check( WorldMovementGridMeshComponent != nullptr );
	WorldMovementGridMeshComponent->SetRelativeScale3D( WorldScaleFactorVector * VREd::GridScaleMultiplier->GetFloat() );
	WorldMovementGridMeshComponent->SetRelativeLocation( WorldScaleFactorVector * FVector( 0.0f, 0.0f, VREd::GridHeightOffset->GetFloat() ) );

	// Update the user scale indicator //@todo
	{
		UVREditorInteractor* LeftHandInteractor = VRMode->GetHandInteractor( EControllerHand::Left );
		UVREditorInteractor* RightHandInteractor = VRMode->GetHandInteractor( EControllerHand::Right );

		if (GetDefault<UVRModeSettings>()->bShowWorldScaleProgressBar &&
			LeftHandInteractor != nullptr && RightHandInteractor != nullptr &&
			 ( ( LeftHandInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::World && RightHandInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::AssistingDrag ) ||
			   ( LeftHandInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::AssistingDrag && RightHandInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::World ) ) )
		{
			// Setting all components to be visible
			CurrentScaleProgressMeshComponent->SetVisibility( true );
			ScaleProgressMeshComponent->SetVisibility( true );
			UserScaleIndicatorText->SetVisibility( true );

			// Here we calculate the distance, direction and center of the two hands
			const FVector ScaleIndicatorStartPosition = LeftHandInteractor->GetTransform( ).GetLocation( );
			const FVector ScaleIndicatorEndPosition = RightHandInteractor->GetTransform( ).GetLocation( );
			const FVector ScaleIndicatorDirection = ( ScaleIndicatorEndPosition - ScaleIndicatorStartPosition ).GetSafeNormal( );

			const float ScaleIndicatorLength = FMath::Max( 0.000001f, (float)( ScaleIndicatorEndPosition - ScaleIndicatorStartPosition ).Size() );
			const float Radius = ( VREd::ScaleProgressBarRadius->GetFloat( ) * WorldScaleFactor );
			const float ProgressbarLength = VREd::ScaleProgressBarLength->GetFloat( );
			const float Scale = WorldScaleFactor;

			// Add an offset to the center, we don't want the hands to clip through the hand meshes
			FVector MiddleLocation = ScaleIndicatorEndPosition - ( ScaleIndicatorLength * 0.5f ) * ScaleIndicatorDirection;
			MiddleLocation.Z += Scale * 5.f;

			// Setting the transform for the scale progressbar
			{
				const FVector ProgressbarStart = MiddleLocation - ( ( ProgressbarLength * 0.5f ) * Scale ) * ScaleIndicatorDirection;

				ScaleProgressMeshComponent->SetWorldTransform( FTransform( ScaleIndicatorDirection.ToOrientationRotator( ),
					ProgressbarStart,
					FVector( ProgressbarLength * Scale, Radius, Radius )
					) );
			}

			// Setting the transform for the current scale progressbar from the center
			{
				const float CurrentProgressScale = ( Scale * Scale ) * ( ProgressbarLength / ( VRMode->GetWorldInteraction().GetMaxScale( ) / 100 ) );
				const FVector CurrentProgressStart = MiddleLocation - ( CurrentProgressScale * 0.5f ) * ScaleIndicatorDirection;

				CurrentScaleProgressMeshComponent->SetWorldTransform( FTransform( ScaleIndicatorDirection.ToOrientationRotator( ),
					CurrentProgressStart,
					FVector( CurrentProgressScale, Radius * 2, Radius * 2 )
					) );
			}

			//Setting the transform for the scale text
			{
				MiddleLocation.Z += Scale * 5.0f;
				UserScaleIndicatorText->SetWorldTransform( FTransform( ( VRMode->GetHeadTransform().GetLocation( ) - MiddleLocation ).ToOrientationRotator( ),
					MiddleLocation,
					VRMode->GetRoomSpaceHeadTransform().GetScale3D()  * Scale
					) );

				FNumberFormattingOptions NumberFormat;
				NumberFormat.MinimumIntegralDigits = 1;
				NumberFormat.MaximumIntegralDigits = 10000;
				NumberFormat.MinimumFractionalDigits = 1;
				NumberFormat.MaximumFractionalDigits = 1;
				UserScaleIndicatorText->SetText( FText::AsNumber( Scale, &NumberFormat ) );
			}
		}
		else
		{
			//Setting all components invisible
			CurrentScaleProgressMeshComponent->SetVisibility( false );
			ScaleProgressMeshComponent->SetVisibility( false );
			UserScaleIndicatorText->SetVisibility( false );
		}
	}

	// Updating the opacity and visibility of the grid according to the controllers //@todo
	{
		if (GetDefault<UVRModeSettings>()->bShowWorldMovementGrid)
		{
			UVREditorInteractor* LeftHandInteractor = VRMode->GetHandInteractor( EControllerHand::Left );
			UVREditorInteractor* RightHandInteractor = VRMode->GetHandInteractor( EControllerHand::Right );

			if ( LeftHandInteractor != nullptr && RightHandInteractor )
			{
				// Show the grid full opacity when dragging or scaling
				float GoalOpacity = 0.f;
				if ( LeftHandInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::World || RightHandInteractor->GetDraggingMode() == EViewportInteractionDraggingMode::World )
				{
					GoalOpacity = 1.0f;
				}
				else if ( ( LeftHandInteractor->GetLastDraggingMode() == EViewportInteractionDraggingMode::World && !LeftHandInteractor->GetDragTranslationVelocity().IsNearlyZero( VREd::GridMovementTolerance->GetFloat() ) ) )
				{
					GoalOpacity = FMath::Clamp( (float)LeftHandInteractor->GetDragTranslationVelocity().Size() / VREd::GridFadeStartVelocity->GetFloat(), 0.0f, 1.0f );
				}
				else if ( ( RightHandInteractor->GetLastDraggingMode() == EViewportInteractionDraggingMode::World && !RightHandInteractor->GetDragTranslationVelocity().IsNearlyZero( VREd::GridMovementTolerance->GetFloat() ) ) )
				{
					GoalOpacity = FMath::Clamp( (float)RightHandInteractor->GetDragTranslationVelocity().Size() / VREd::GridFadeStartVelocity->GetFloat(), 0.0f, 1.0f );
				}

				// Check the current opacity and add or subtract to reach the goal
				float NewOpacity = WorldMovementGridOpacity;
				if ( NewOpacity < GoalOpacity )
				{
					NewOpacity = FMath::Min( GoalOpacity, NewOpacity + DeltaTime * VREd::GridFadeMultiplier->GetFloat() );
				}
				else if ( NewOpacity > GoalOpacity )
				{
					NewOpacity = FMath::Max( GoalOpacity, NewOpacity - DeltaTime * VREd::GridFadeMultiplier->GetFloat() );
				}

				// Only update when the opacity is different
				if ( !FMath::IsNearlyEqual( WorldMovementGridOpacity, GoalOpacity ) )
				{
					WorldMovementGridOpacity = NewOpacity;

					// See if the opacity is almost zero and if the goal opacity is lower than the new opacity set it to zero if so. Otherwise it will flicker
					if ( WorldMovementGridOpacity < SMALL_NUMBER )
					{
						WorldMovementGridOpacity = 0.f;
						WorldMovementGridMeshComponent->SetVisibility( false );
					}
					else
					{
						WorldMovementGridMeshComponent->SetVisibility( true );
					}

					static const FName OpacityName( "OpacityParam" );
					WorldMovementGridMID->SetScalarParameterValue( OpacityName, WorldMovementGridOpacity * VREd::GridMaxOpacity->GetFloat() );
				}
			}
		}
		else
		{
			WorldMovementGridMeshComponent->SetVisibility( false );
		}
	}

	// Apply a post process effect while the user is moving the world around.  The effect "greys out" any pixels
	// that are not nearby, and completely hides the skybox and other very distant objects.  This is to help
	// prevent simulation sickness when moving/rotating/scaling the entire world around them.
	{
		PostProcessComponent->bEnabled = GetDefault<UVRModeSettings>()->bShowWorldMovementPostProcess;

		// Make sure our world size is reflected in the post process material
		static FName WorldScaleFactorParameterName( "WorldScaleFactor" );
		WorldMovementPostProcessMaterial->SetScalarParameterValue( WorldScaleFactorParameterName, WorldScaleFactor );

		static FName RoomOriginParameterName( "RoomOrigin" );
		WorldMovementPostProcessMaterial->SetVectorParameterValue( RoomOriginParameterName, VRMode->GetRoomTransform( ).GetLocation( ) );

		static FName FogColorParameterName( "FogColor" );
		// WorldMovementPostProcessMaterial->SetVectorParameterValue( FogColorParameterName, FLinearColor::Black );

		static FName StartDistanceParameterName( "StartDistance" );
		WorldMovementPostProcessMaterial->SetScalarParameterValue( StartDistanceParameterName, VREd::WorldMovementFogStartDistance->GetFloat( ) );

		static FName EndDistanceParameterName( "EndDistance" );
		WorldMovementPostProcessMaterial->SetScalarParameterValue( EndDistanceParameterName, VREd::WorldMovementFogEndDistance->GetFloat( ) );

		static FName FogOpacityParameterName( "FogOpacity" );
		WorldMovementPostProcessMaterial->SetScalarParameterValue( FogOpacityParameterName, VREd::WorldMovementFogOpacity->GetFloat( ) );

		static FName SkyboxDistanceParameterName( "SkyboxDistance" );
		WorldMovementPostProcessMaterial->SetScalarParameterValue( SkyboxDistanceParameterName, VREd::WorldMovementFogSkyboxDistance->GetFloat( ) );

		const bool bShouldDrawWorldMovementPostProcess = WorldMovementGridOpacity > KINDA_SMALL_NUMBER;
		if ( bShouldDrawWorldMovementPostProcess != bIsDrawingWorldMovementPostProcess )
		{
			UPostProcessComponent* PostProcess = PostProcessComponent;
			if ( bShouldDrawWorldMovementPostProcess )
			{
				PostProcess->AddOrUpdateBlendable( ToRawPtr(WorldMovementPostProcessMaterial) );
				bIsDrawingWorldMovementPostProcess = true;
			}
			else
			{
				bIsDrawingWorldMovementPostProcess = false;
				PostProcess->Settings.RemoveBlendable( ToRawPtr(WorldMovementPostProcessMaterial) );
			}
		}

		if ( bIsDrawingWorldMovementPostProcess )
		{
			static FName OpacityParameterName( "Opacity" );
			WorldMovementPostProcessMaterial->SetScalarParameterValue( OpacityParameterName, FMath::Clamp( WorldMovementGridOpacity, 0.0f, 1.0f ) );
		}
	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
