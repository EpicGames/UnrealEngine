// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "VREditorAvatarActor.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UMaterialInstanceDynamic;

/**
 * Avatar Actor
 */
UCLASS()
class AVREditorAvatarActor : public AActor
{
	GENERATED_BODY()

public:

	/** Default constructor */
	AVREditorAvatarActor();

	/** Called by VREditorMode::Enter to initialize all post constructor components and to set the VRMode */
	void Init( class UVREditorMode* InVRMode );

	/** Called by VREditorMode to update us every frame */
	void TickManually( const float DeltaTime );

	// UObject overrides
	virtual bool IsEditorOnly() const final
	{
		return true;
	}
	

	virtual bool HasLocalNetOwner() const override
	{
		// TODO: When using Multi-User with two VRMode editors, we might have to do something else here for when this is called by MotionControllerComponent::PollControllerState()
		return true;
	}

private:

	/** Our avatar's head mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> HeadMeshComponent;

	/** The grid that appears while the user is dragging the world around */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> WorldMovementGridMeshComponent;

	//
	// World movement grid & FX
	//

	/** Grid mesh component dynamic material instance to set the opacity */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> WorldMovementGridMID;

	/** Opacity of the movement grid and post process */
	UPROPERTY()
	float WorldMovementGridOpacity;

	/** True if we're currently drawing our world movement post process */
	UPROPERTY()
	bool bIsDrawingWorldMovementPostProcess;

	/** Post process material for "greying out" the world while in world movement mode */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> WorldMovementPostProcessMaterial;

	//
	// World scaling progress bar
	//

	/** Background progressbar scaling mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> ScaleProgressMeshComponent;

	/** Current scale progressbar mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> CurrentScaleProgressMeshComponent;

	/** Current scale text */
	UPROPERTY()
	TObjectPtr<class UTextRenderComponent> UserScaleIndicatorText;

	/** Base dynamic material for the user scale fixed progressbar */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> FixedUserScaleMID;

	/** Translucent dynamic material for the user scale fixed progressbar */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> TranslucentFixedUserScaleMID;
	
	/** Base dynamic material for the current user scale progressbar */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> CurrentUserScaleMID;

	/** Translucent dynamic material for the current user scale progressbar */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> TranslucentCurrentUserScaleMID;

	//
	// Post process
	//

	/** Post process for drawing VR-specific post effects */
	UPROPERTY()
	TObjectPtr<class UPostProcessComponent> PostProcessComponent;

	/** Owning object */
	UPROPERTY()
	TObjectPtr<class UVREditorMode> VRMode;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
