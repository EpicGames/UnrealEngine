// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorBaseActor.h"
#include "VREditorMode.h"
#include "VREditorInteractor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorBaseActor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable UIOnHandRotationOffset( TEXT( "VREd.UIOnHandRotationOffset" ), 45.0f, TEXT( "Rotation offset for UI that's docked to your hand, to make it more comfortable to hold" ) );
	static FAutoConsoleVariable UIOnArmRotationOffset( TEXT( "VREd.UIOnArmRotationOffset" ), 0.0f, TEXT( "Rotation offset for UI that's docked to your arm, so it aligns with the controllers" ) );
	static FAutoConsoleVariable DockUISmoothingAmount( TEXT( "VREd.DockUISmoothingAmount" ), 0.75f, TEXT( "How much to smooth out UI transforms (frame rate sensitive)" ) );
}

AVREditorBaseActor::AVREditorBaseActor() 
	: Super(),
	Scale( 1.0f ),
	VRMode(nullptr),
	LocalRotation( FRotator( 90.0f, 180.0f, 0.0f ) ),
	RelativeOffset( FVector::ZeroVector ),
	DockedTo( EDockedTo::Nothing ),
	PreviousDockedTo( EDockedTo::Nothing ),
	bIsMoving( false ),
	MoveToTransform( FTransform() ),
	StartMoveToTransform( FTransform() ),
	MoveToAlpha( 0.0f ),
	MoveToTime( 0.0f ),
	MoveToResultDock( EDockedTo::Nothing ),
	LastDockedUIToWorld()
{
	bIsEditorOnlyActor = true;
}

void AVREditorBaseActor::SetVRMode( UVREditorMode* InVRMode )
{
	VRMode = InVRMode;
}

void AVREditorBaseActor::TickManually( float DeltaTime )
{
	Super::Tick( DeltaTime );

	// Update fading state
	UpdateFadingState( DeltaTime );

	if ( bIsMoving )
	{
		TickMoveTo( DeltaTime );
	}
	else
	{
		// Update transform
		UpdateTransformIfDocked();
	}
}

void AVREditorBaseActor::SetRelativeOffset( const FVector& InRelativeOffset )
{
	RelativeOffset = InRelativeOffset;
}

void AVREditorBaseActor::SetLocalRotation( const FRotator& InLocalRotation )
{
	LocalRotation = InLocalRotation;
}

void AVREditorBaseActor::SetDockedTo( const EDockedTo NewDockedTo )
{
	PreviousDockedTo = DockedTo;
	DockedTo = NewDockedTo;
	UpdateTransformIfDocked();
}

void AVREditorBaseActor::MoveTo( const FTransform& ResultTransform, const float TotalMoveToTime, const EDockedTo ResultDock )
{
	MoveToTime = TotalMoveToTime;
	bIsMoving = true;
	StartMoveToTransform = GetActorTransform();
	MoveToTransform = ResultTransform;
	MoveToAlpha = 0.0f;
	MoveToResultDock = ResultDock;
}

void AVREditorBaseActor::StopMoveTo()
{
	bIsMoving = false;
	SetTransform( MoveToTransform );
	MoveToTransform = FTransform();
}


FTransform AVREditorBaseActor::MakeUITransformLockedToHand( UViewportInteractor* Interactor, const bool bOnArm, const FVector& InRelativeOffset, const FRotator& InLocalRotation )
{
	const float WorldScaleFactor = VRMode->GetWorldScaleFactor();

	FTransform UIToHandTransform( InLocalRotation, InRelativeOffset * WorldScaleFactor );
	// If the UI is on the hand
	if ( !bOnArm )
	{
		UIToHandTransform *= FTransform( FRotator( VREd::UIOnHandRotationOffset->GetFloat(), 0.0f, 0.0f ).Quaternion(), FVector::ZeroVector );
	}
	// If the UI is on the arm
	else
	{
		UIToHandTransform *= FTransform( FRotator( VREd::UIOnArmRotationOffset->GetFloat(), 0.0f, 0.0f ).Quaternion(), FVector::ZeroVector );
	}

	const FTransform HandToWorldTransform = Interactor->GetTransform();
	FTransform UIToWorldTransform = UIToHandTransform * HandToWorldTransform;
	UIToWorldTransform.SetScale3D( FVector( Scale * WorldScaleFactor ) );

	return UIToWorldTransform;
}

void AVREditorBaseActor::UpdateTransformIfDocked()
{
	if ( DockedTo != EDockedTo::Nothing && DockedTo != EDockedTo::Dragging )
	{
		FTransform NewTransform;

		if ( DockedTo == EDockedTo::LeftHand )
		{
			const bool bOnArm = false;
			NewTransform = MakeUITransformLockedToHand( VRMode->GetHandInteractor( EControllerHand::Left ), bOnArm );
		}
		else if ( DockedTo == EDockedTo::RightHand )
		{
			const bool bOnArm = false;
			NewTransform = MakeUITransformLockedToHand( VRMode->GetHandInteractor( EControllerHand::Right ), bOnArm );
		}
		else if ( DockedTo == EDockedTo::LeftArm )
		{
			const bool bOnArm = true;
			NewTransform = MakeUITransformLockedToHand( VRMode->GetHandInteractor( EControllerHand::Left ), bOnArm );
		}
		else if ( DockedTo == EDockedTo::RightArm )
		{
			const bool bOnArm = true;
			NewTransform = MakeUITransformLockedToHand( VRMode->GetHandInteractor( EControllerHand::Right ), bOnArm );
		}
		else if ( DockedTo == EDockedTo::Room )
		{
			NewTransform = MakeUITransformLockedToRoom();
		}
		else if ( DockedTo == EDockedTo::Custom )
		{
			NewTransform = MakeCustomUITransform();
		}
		else
		{
			check( 0 );
		}

		// Smooth out the transform
		if( DockedTo != EDockedTo::Custom && DockedTo != EDockedTo::Room )
		{
			if( LastDockedUIToWorld.IsSet() )
			{
				FTransform SmoothedDockedUIToWorld;
				SmoothedDockedUIToWorld.Blend( NewTransform, LastDockedUIToWorld.GetValue(), VREd::DockUISmoothingAmount->GetFloat() );
				NewTransform = SmoothedDockedUIToWorld;
			}

			LastDockedUIToWorld = NewTransform;
		}
		else
		{
			LastDockedUIToWorld.Reset();
		}

		SetTransform( NewTransform );
	}
	else
	{
		LastDockedUIToWorld.Reset();
	}
}

FTransform AVREditorBaseActor::MakeUITransformLockedToHand( UViewportInteractor* Interactor, const bool bOnArm )
{
	return MakeUITransformLockedToHand( Interactor, bOnArm, RelativeOffset, LocalRotation );
}

FTransform AVREditorBaseActor::MakeUITransformLockedToRoom()
{
	const float WorldScaleFactor = VRMode->GetWorldScaleFactor();

	const FTransform UIToRoomTransform( LocalRotation, RelativeOffset * WorldScaleFactor );

	const FTransform RoomToWorldTransform = VRMode->GetRoomTransform();

	FTransform UIToWorldTransform = UIToRoomTransform * RoomToWorldTransform;
	UIToWorldTransform.SetScale3D( FVector( Scale * WorldScaleFactor ) );

	return UIToWorldTransform;
}

void AVREditorBaseActor::TickMoveTo( const float DeltaTime )
{
	const float WorldScaleFactor = VRMode->GetWorldScaleFactor();

	MoveToAlpha += DeltaTime;
	const float LerpTime = MoveToTime;
	if ( MoveToAlpha > MoveToTime )
	{
		MoveToAlpha = MoveToTime;
		bIsMoving = false;
		SetDockedTo( MoveToResultDock );
	}

	const float CurrentALpha = MoveToAlpha / LerpTime;
	const FVector NewLocation = FMath::Lerp( StartMoveToTransform.GetLocation(), MoveToTransform.GetLocation(), CurrentALpha );
	const FQuat NewRotation = FMath::Lerp( StartMoveToTransform.GetRotation(), MoveToTransform.GetRotation(), CurrentALpha );
	FTransform NewTransform( NewRotation, NewLocation, FVector( Scale * WorldScaleFactor ) );

	SetTransform( NewTransform );
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
