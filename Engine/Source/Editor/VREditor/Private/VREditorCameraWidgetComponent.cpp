// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorCameraWidgetComponent.h"
#include "IVREditorModule.h"
#include "VREditorAssetContainer.h"
#include "VREditorMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorCameraWidgetComponent)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UVREditorCameraWidgetComponent::UVREditorCameraWidgetComponent()
	: Super()	
{	
	// Override this shader for VR camera viewfinders so that we get color-correct images.
	 //This shader does an sRGB -> Linear conversion and doesn't apply the "UI Brightness" setting .

	if (UNLIKELY(IsRunningDedicatedServer()) || HasAnyFlags(RF_ClassDefaultObject))   // @todo vreditor: early out to avoid loading font assets in the cooker on Linux
	{
		return;
	}

	if (!IsRunningCommandlet())
	{
		if (IVREditorModule::IsAvailable())
		{
			IVREditorModule& VRModule = IVREditorModule::Get();

			if (UVREditorMode* VRMode = VRModule.GetVRMode())
			{
				const UVREditorAssetContainer& VRAssetContainer = VRMode->GetAssetContainer();
				{
					OpaqueMaterial_OneSided = VRAssetContainer.CameraWidgetMaterial;
				}
			}
		}

	}
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
