// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VREditorWidgetComponent.h"
#include "VREditorCameraWidgetComponent.generated.h"

/**
 * A specialized WidgetComponent that color-corrects cameras previews (viewfinders) in VR
 */
UCLASS(hidedropdown)
class UVREditorCameraWidgetComponent : public UVREditorWidgetComponent
{
	GENERATED_BODY()

public:	
	UVREditorCameraWidgetComponent();
	
};
