// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorFloatingText.h"
#include "VREditorMode.h"
#include "VREditorAssetContainer.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/World.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/Material.h"
#include "Engine/Font.h"
#include "Engine/StaticMesh.h"
#include "Engine/CollisionProfile.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "GameFramework/WorldSettings.h"
#include "Components/TextRenderComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorFloatingText)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


AFloatingText::AFloatingText()
{
	// Create root default scene component
	{
		SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));
		check(SceneComponent != nullptr);

		RootComponent = SceneComponent;
	}
}


void AFloatingText::PostActorCreated()
{
	Super::PostActorCreated();

	// @todo vreditor: Tweak
	const bool bAllowTextLighting = false;
	const float TextSize = 1.5f;

	const UVREditorAssetContainer& AssetContainer = UVREditorMode::LoadAssetContainer();

	{
		FirstLineComponent = NewObject<UStaticMeshComponent>(this, TEXT("FirstLine"));
		check(FirstLineComponent != nullptr);

		FirstLineComponent->SetStaticMesh(AssetContainer.LineSegmentCylinderMesh);
		FirstLineComponent->SetMobility(EComponentMobility::Movable);
		FirstLineComponent->SetupAttachment(SceneComponent);
		FirstLineComponent->RegisterComponent();
		FirstLineComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

		FirstLineComponent->SetGenerateOverlapEvents(false);
		FirstLineComponent->SetCanEverAffectNavigation(false);
		FirstLineComponent->bCastDynamicShadow = bAllowTextLighting;
		FirstLineComponent->bCastStaticShadow = false;
		FirstLineComponent->bAffectDistanceFieldLighting = bAllowTextLighting;
		FirstLineComponent->bAffectDynamicIndirectLighting = bAllowTextLighting;
	}

	{
		JointSphereComponent = NewObject<UStaticMeshComponent>(this, TEXT("JointSphere"));
		check(JointSphereComponent != nullptr);

		JointSphereComponent->SetStaticMesh(AssetContainer.JointSphereMesh);
		JointSphereComponent->SetMobility(EComponentMobility::Movable);
		JointSphereComponent->SetupAttachment(SceneComponent);
		JointSphereComponent->RegisterComponent();
		JointSphereComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

		JointSphereComponent->SetGenerateOverlapEvents(false);
		JointSphereComponent->SetCanEverAffectNavigation(false);
		JointSphereComponent->bCastDynamicShadow = bAllowTextLighting;
		JointSphereComponent->bCastStaticShadow = false;
		JointSphereComponent->bAffectDistanceFieldLighting = bAllowTextLighting;
		JointSphereComponent->bAffectDynamicIndirectLighting = bAllowTextLighting;

	}

	{
		SecondLineComponent = NewObject<UStaticMeshComponent>(this, TEXT("SecondLine"));
		check(SecondLineComponent != nullptr);

		SecondLineComponent->SetStaticMesh(AssetContainer.LineSegmentCylinderMesh);
		SecondLineComponent->SetMobility(EComponentMobility::Movable);
		SecondLineComponent->SetupAttachment(SceneComponent);
		SecondLineComponent->RegisterComponent();
		SecondLineComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

		SecondLineComponent->SetGenerateOverlapEvents(false);
		SecondLineComponent->SetCanEverAffectNavigation(false);
		SecondLineComponent->bCastDynamicShadow = bAllowTextLighting;
		SecondLineComponent->bCastStaticShadow = false;
		SecondLineComponent->bAffectDistanceFieldLighting = bAllowTextLighting;
		SecondLineComponent->bAffectDynamicIndirectLighting = bAllowTextLighting;

	}

	LineMaterial = AssetContainer.LineMaterial;
	MaskedTextMaterial = AssetContainer.TextMaterial;
	TranslucentTextMaterial = AssetContainer.TranslucentTextMaterial;

	{
		TextComponent = NewObject<UTextRenderComponent>(this, TEXT("Text"));
		check(TextComponent != nullptr);

		TextComponent->SetMobility(EComponentMobility::Movable);
		TextComponent->SetupAttachment(SceneComponent);
		TextComponent->RegisterComponent();
		TextComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

		TextComponent->SetGenerateOverlapEvents(false);
		TextComponent->SetCanEverAffectNavigation(false);
		TextComponent->bCastDynamicShadow = bAllowTextLighting;
		TextComponent->bCastStaticShadow = false;
		TextComponent->bAffectDistanceFieldLighting = bAllowTextLighting;
		TextComponent->bAffectDynamicIndirectLighting = bAllowTextLighting;


		TextComponent->SetWorldSize(TextSize);

		// Use a custom font.  The text will be visible up close.	   
		TextComponent->SetFont(AssetContainer.TextFont);

		if (MaskedTextMaterial != nullptr)
		{
			// Assign our custom text rendering material.
			TextComponent->SetTextMaterial(MaskedTextMaterial);
		}
		TextComponent->SetTextRenderColor(FLinearColor::White.ToFColor(false));

		// Left justify the text
		TextComponent->SetHorizontalAlignment(EHTA_Left);

	}

	// Create an MID so that we can change parameters on the fly (fading)
	if (LineMaterial != nullptr)
	{
		this->LineMaterialMID = UMaterialInstanceDynamic::Create(LineMaterial, this);

		FirstLineComponent->SetMaterial(0, LineMaterialMID);
		JointSphereComponent->SetMaterial(0, LineMaterialMID);
		SecondLineComponent->SetMaterial(0, LineMaterialMID);
	}
}


void AFloatingText::SetText( const FText& NewText )
{
	check( TextComponent != nullptr );
	TextComponent->SetText( NewText );
}


void AFloatingText::SetOpacity( const float NewOpacity )
{
	const FLinearColor NewColor = FLinearColor( 0.6f, 0.6f, 0.6f ).CopyWithNewOpacity( NewOpacity );	// @todo vreditor: Tweak brightness
	const FColor NewFColor = NewColor.ToFColor( false );

	check( TextComponent != nullptr );
// 	if( NewOpacity >= 1.0f - KINDA_SMALL_NUMBER )	// @todo vreditor ui: get fading/translucency working again!
// 	{
		if( TextComponent->GetMaterial( 0 ) != MaskedTextMaterial )
		{
			TextComponent->SetTextMaterial( MaskedTextMaterial );
		}
// 	}
// 	else
// 	{
// 		if( TextComponent->GetMaterial( 0 ) != TranslucentTextMaterial )
// 		{
// 			TextComponent->SetTextMaterial( TranslucentTextMaterial );
// 		}
// 	}
	
	if( NewFColor != TextComponent->TextRenderColor )
	{
		TextComponent->SetTextRenderColor( NewFColor );
	}

	check( LineMaterialMID != nullptr );
	static FName ColorAndOpacityParameterName( "ColorAndOpacity" );
	LineMaterialMID->SetVectorParameterValue( ColorAndOpacityParameterName, NewColor );
}


void AFloatingText::Update( const FVector OrientateToward )
{
	// Orientate it toward the viewer
	const FVector DirectionToward = ( OrientateToward - GetActorLocation() ).GetSafeNormal();

	const FQuat TowardRotation = DirectionToward.ToOrientationQuat();

	// @todo vreditor tweak
	const float LineRadius = 0.1f;
	const float FirstLineLength = 4.0f;	   // Default line length (note that socket scale can affect this!)
	const float SecondLineLength = (float) TextComponent->GetTextLocalSize().Y;	// The second line "underlines" the text


	// NOTE: The origin of the actor will be the designated target of the text
	const FVector FirstLineLocation = FVector::ZeroVector;
	const FQuat FirstLineRotation = FVector::ForwardVector.ToOrientationQuat();
	const FVector FirstLineScale = FVector( FirstLineLength, LineRadius, LineRadius );
	FirstLineComponent->SetRelativeLocation( FirstLineLocation );
	FirstLineComponent->SetRelativeRotation( FirstLineRotation );
	FirstLineComponent->SetRelativeScale3D( FirstLineScale );

	// NOTE: The joint sphere draws at the connection point between the lines
	const FVector JointLocation = FirstLineLocation + FirstLineRotation * FVector::ForwardVector * FirstLineLength;
	const FVector JointScale = FVector( LineRadius );
	JointSphereComponent->SetRelativeLocation( JointLocation );
	JointSphereComponent->SetRelativeScale3D( JointScale );

	// NOTE: The second line starts at the joint location
	SecondLineComponent->SetWorldLocation( JointSphereComponent->GetComponentLocation() );
	SecondLineComponent->SetWorldRotation( ( TowardRotation * -FVector::RightVector ).ToOrientationQuat() );
	SecondLineComponent->SetRelativeScale3D( FVector( ( SecondLineLength / GetActorScale().X ) * GetWorld()->GetWorldSettings()->WorldToMeters / 100.0f, LineRadius, LineRadius ) );

	TextComponent->SetWorldLocation( JointSphereComponent->GetComponentLocation() );
	TextComponent->SetWorldRotation( ( TowardRotation * FVector::ForwardVector ).ToOrientationQuat() );

}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
