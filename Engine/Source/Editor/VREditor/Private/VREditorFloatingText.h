// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "VREditorFloatingText.generated.h"

/**
 * Draws 3D text in the world along with targeting line cues
 */
UCLASS(NotPlaceable, NotBlueprintable)
class AFloatingText : public AActor
{
	GENERATED_BODY()

public:
	
	/** Default constructor that sets up CDO properties */
	AFloatingText();

	// AActor overrides
	virtual void PostActorCreated() override;
	virtual bool IsEditorOnly() const final
	{
		return true;
	}

	/** Sets the text to display */
	void SetText( const FText& NewText );

	/** Sets the opacity of the actor */
	void SetOpacity( const float Opacity );

	/** Call this every frame to orientate the text toward the specified transform */
	void Update( const FVector OrientateToward );


private:

	/** Scene component root of this actor */
	UPROPERTY()
	TObjectPtr<class USceneComponent> SceneComponent;

	/** First line segment component.  Starts at the designation location, goes toward the line connection point. */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> FirstLineComponent;

	/** Sphere that connects the two line segments and makes the joint look smooth and round */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> JointSphereComponent;

	/** Second line segment component.  Starts at the connection point and goes toward the 3D text. */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> SecondLineComponent;

	/** The 3D text we're drawing.  Positioned at the end of the second line. */
	UPROPERTY()
	TObjectPtr<class UTextRenderComponent> TextComponent;

	/** Masked text material.  Used after faded in */
	UPROPERTY()
	TObjectPtr<class UMaterialInterface> MaskedTextMaterial;

	/** Translucent text material.  Used during fading */
	UPROPERTY()
	TObjectPtr<class UMaterialInterface> TranslucentTextMaterial;

	/** Material to use for the line meshes */
	UPROPERTY()
	TObjectPtr<class UMaterialInterface> LineMaterial;

	/** Dynamic material instance for fading lines in and out */
	UPROPERTY( transient )
	TObjectPtr<class UMaterialInstanceDynamic> LineMaterialMID;

};
