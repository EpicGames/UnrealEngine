// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorInteractor.h"

#include "ActorTransformer.h"
#include "Components/PointLightComponent.h"
#include "Components/SplineComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Engine/Selection.h"
#include "Engine/StaticMeshSocket.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "IMotionController.h"
#include "IXRTrackingSystem.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MotionControllerComponent.h"
#include "ViewportWorldInteraction.h"
#include "VREditorActions.h"
#include "VREditorAssetContainer.h"
#include "VREditorMode.h"
#include "UI/VREditorDockableWindow.h"
#include "VREditorFloatingText.h"
#include "VREditorModule.h"
#include "UI/VREditorFloatingUI.h"
#include "VREditorPlacement.h"
#include "UI/VREditorRadialFloatingUI.h"
#include "UI/VREditorUISystem.h"
#include "VRModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorInteractor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


namespace VREd
{
	static FAutoConsoleVariable TriggerTouchThreshold_Vive( TEXT( "VI.TriggerTouchThreshold_Vive" ), 0.025f, TEXT( "Minimum trigger threshold before we consider the trigger 'touched'" ) );
	static FAutoConsoleVariable TriggerTouchThreshold_Rift( TEXT( "VI.TriggerTouchThreshold_Rift" ), 0.15f, TEXT( "Minimum trigger threshold before we consider the trigger 'touched'" ) );
	static FAutoConsoleVariable TriggerDeadZone_Vive( TEXT( "VI.TriggerDeadZone_Vive" ), 0.25f, TEXT( "Trigger dead zone.  The trigger must be fully released before we'll trigger a new 'light press'" ) );
	static FAutoConsoleVariable TriggerDeadZone_Rift( TEXT( "VI.TriggerDeadZone_Rift" ), 0.25f, TEXT( "Trigger dead zone.  The trigger must be fully released before we'll trigger a new 'light press'" ) );
	static FAutoConsoleVariable TriggerFullyPressedThreshold_Vive( TEXT( "VI.TriggerFullyPressedThreshold_Vive" ), 0.90f, TEXT( "Minimum trigger threshold before we consider the trigger 'fully pressed'" ) );
	static FAutoConsoleVariable TriggerFullyPressedThreshold_Rift( TEXT( "VI.TriggerFullyPressedThreshold_Rift" ), 0.99f, TEXT( "Minimum trigger threshold before we consider the trigger 'fully pressed'" ) );

	//Laser
	static FAutoConsoleVariable OculusLaserPointerRotationOffset( TEXT( "VI.OculusLaserPointerRotationOffset" ), 0.0f, TEXT( "How much to rotate the laser pointer (pitch) relative to the forward vector of the controller (Oculus)" ) );
	static FAutoConsoleVariable ViveLaserPointerRotationOffset( TEXT( "VI.ViveLaserPointerRotationOffset" ), /* -57.8f */ 0.0f, TEXT( "How much to rotate the laser pointer (pitch) relative to the forward vector of the controller (Vive)" ) );
	static FAutoConsoleVariable OculusLaserPointerStartOffset( TEXT( "VI.OculusLaserPointerStartOffset" ), 2.8f, TEXT( "How far to offset the start of the laser pointer to avoid overlapping the hand mesh geometry (Oculus)" ) );
	static FAutoConsoleVariable ViveLaserPointerStartOffset( TEXT( "VI.ViveLaserPointerStartOffset" ), 1.25f /* 8.5f */, TEXT( "How far to offset the start of the laser pointer to avoid overlapping the hand mesh geometry (Vive)" ) );

	// Laser visuals
	static FAutoConsoleVariable LaserPointerLightRadius( TEXT( "VREd.LaserPointLightRadius" ), 10.0f, TEXT( "How big our hover light is" ) );
	static FAutoConsoleVariable LaserPointerRadius( TEXT( "VREd.LaserPointerRadius" ), .5f, TEXT( "Radius of the laser pointer line" ) );
	static FAutoConsoleVariable LaserPointerHoverBallRadius( TEXT( "VREd.LaserPointerHoverBallRadius" ), 1.0f, TEXT( "Radius of the visual cue for a hovered object along the laser pointer ray" ) );
	static FAutoConsoleVariable LaserPointerLightPullBackDistance( TEXT( "VREd.LaserPointerLightPullBackDistance" ), 2.5f, TEXT( "How far to pull back our little hover light from the impact surface" ) );
	static FAutoConsoleVariable LaserRadiusScaleWhenOverUI( TEXT( "VREd.LaserRadiusScaleWhenOverUI" ), 0.25f, TEXT( "How much to scale down the size of the laser pointer radius when over UI" ) );
	static FAutoConsoleVariable HoverBallRadiusScaleWhenOverUI( TEXT( "VREd.HoverBallRadiusScaleWhenOverUI" ), 0.4f, TEXT( "How much to scale down the size of the hover ball when over UI" ) );

	static FAutoConsoleVariable MinTrackpadOffsetBeforeRadialMenu( TEXT( "VREd.MinTrackpadOffsetBeforeRadialMenu" ), 0.5f, TEXT( "How far you have to hold the trackpad upward before you can placing objects instantly by pulling the trigger" ) );
	static FAutoConsoleVariable MinJoystickOffsetBeforeFlick( TEXT( "VREd.MinJoystickOffsetBeforeFlick" ), 0.4f, TEXT( "Dead zone for flick actions on the motion controller" ) );

	static FAutoConsoleVariable TrackpadStopImpactAtLaserBuffer( TEXT( "VREd.TrackpadStopImpactAtLaserBuffer" ), 0.4f, TEXT( "Required amount to slide with input to stop transforming to end of laser" ) );

	static FAutoConsoleVariable TrackpadAbsoluteDragSpeed( TEXT( "VREd.TrackpadAbsoluteDragSpeed" ), 80.0f, TEXT( "How fast objects move toward or away when you drag on the touchpad while carrying them" ) );
	static FAutoConsoleVariable TrackpadRelativeDragSpeed( TEXT( "VREd.TrackpadRelativeDragSpeed" ), 8.0f, TEXT( "How fast objects move toward or away when you hold a direction on an analog stick while carrying them" ) );
	static FAutoConsoleVariable MinVelocityForInertia( TEXT( "VREd.MinVelocityForMotionControllerInertia" ), 1.0f, TEXT( "Minimum velocity (in cm/frame in unscaled room space) before inertia will kick in when releasing objects (or the world)" ) );

	static FAutoConsoleVariable SequencerScrubMax( TEXT( "VREd.SequencerScrubMax" ), 2.0f, TEXT( "Max fast forward or fast reverse magnitude" ) );

	static FAutoConsoleVariable ShowControllerHelpLabels( TEXT( "VREd.ShowControllerHelpLabels" ), 0, TEXT( "Enables help text overlay when controllers are near the viewer" ) );
	static FAutoConsoleVariable HelpLabelFadeDuration( TEXT( "VREd.HelpLabelFadeDuration" ), 0.4f, TEXT( "Duration to fade controller help labels in and out" ) );
	static FAutoConsoleVariable HelpLabelFadeDistance( TEXT( "VREd.HelpLabelFadeDistance" ), 30.0f, TEXT( "Distance at which controller help labels should appear (in cm)" ) );

	static FAutoConsoleVariable InvertTrackpadVertical( TEXT( "VREd.InvertTrackpadVertical" ), 0, TEXT( "Toggles inverting the touch pad vertical axis" ) );
}

namespace VREditorKeyNames
{
	// @todo vreditor input: Ideally these would not be needed, but SteamVR fires off it's "trigger pressed" event
	// well before the trigger is fully down (*click*)
	static const FName MotionController_Left_PressedTriggerAxis( "MotionController_Left_PressedTriggerAxis" );
	static const FName MotionController_Right_PressedTriggerAxis( "MotionController_Right_PressedTriggerAxis" );
	static const FName MotionController_Left_FullyPressedTriggerAxis( "MotionController_Left_FullyPressedTriggerAxis" );
	static const FName MotionController_Right_FullyPressedTriggerAxis( "MotionController_Right_FullyPressedTriggerAxis" );
}

static const FName OculusDeviceType = TEXT( "OculusHMD" );
static const FName SteamVRDeviceType = TEXT( "SteamVR" );
static const FName OpenXRDeviceType = TEXT( "OpenXR" );

const FName UVREditorInteractor::TrackpadPositionX = FName( "TrackpadPosition_X" );
const FName UVREditorInteractor::TrackpadPositionY = FName( "TrackpadPosition_Y" );
const FName UVREditorInteractor::TriggerAxis = FName( "TriggerAxis" );
const FName UVREditorInteractor::MotionController_Left_PressedTriggerAxis = FName( "MotionController_Left_PressedTriggerAxis" );
const FName UVREditorInteractor::MotionController_Right_PressedTriggerAxis = FName( "MotionController_Right_PressedTriggerAxis" );
const FName UVREditorInteractor::MotionController_Left_FullyPressedTriggerAxis = FName( "MotionController_Left_FullyPressedTriggerAxis" );
const FName UVREditorInteractor::MotionController_Right_FullyPressedTriggerAxis = FName( "MotionController_Right_FullyPressedTriggerAxis" );

#define LOCTEXT_NAMESPACE "UVREditorInteractor"

UVREditorInteractor::UVREditorInteractor() :
	Super(),
	bIsUndoRedoSwipeEnabled( true ),
	MotionControllerComponent( nullptr ),
	LaserMotionControllerComponent( nullptr ),
	HandMeshComponent( nullptr ),
	HandMeshBaseScale( FVector3d::OneVector ),
	HandMeshGripTransform( FTransform::Identity ),
	LaserSplineComponent( nullptr ),
	LaserPointerMID( nullptr ),
	TranslucentLaserPointerMID( nullptr ),
	HoverMeshComponent( nullptr ),
	HoverPointLightComponent( nullptr ),
	HandMeshMID( nullptr ),
	bHaveMotionController( false ),
	bIsModifierPressed( false ),
	SelectAndMoveTriggerValue( 0.0f ),
	LaserStart( FVector::ZeroVector ),
	LaserEnd( FVector::ZeroVector ),
	ControllerType( EControllerType::Unknown ),
	OverrideControllerType( EControllerType::Unknown ),
	bHasUIInFront( false ),
	bHasUIOnForearm( false ),
	bIsClickingOnUI( false ),
	bIsRightClickingOnUI( false ),
	bIsHoveringOverUI( false ),
	UIScrollVelocity( 0.0f ),
	LastUIPressTime( 0.0f ),
	bIsTouchingTrackpad( false ),
	bIsPressingTrackpad( false ),
	TrackpadPosition( FVector2D::ZeroVector ),
	LastTrackpadPosition( FVector2D::ZeroVector ),
	bIsTrackpadPositionValid{ false, false },
	LastTrackpadPositionUpdateTime( FTimespan::MinValue() ),
	LastActiveTrackpadUpdateTime( FTimespan::MinValue() ),
	bForceShowLaser(false),
	ForceLaserColor(),
	bFlickActionExecuted(false),
	bIsScrubbingSequence(false),
	ControllerMotionSource(NAME_None),
	bWantHelpLabels( false ),
	HelpLabelShowOrHideStartTime(FTimespan::MinValue()),
	bIsTriggerFullyPressed( false ),
	bIsTriggerPressed( false ),
	bHasTriggerBeenReleasedSinceLastPress( true ),
	InitialTouchPosition(FVector2D::ZeroVector),
	LastSwipe(ETouchSwipeDirection::None),
	VRMode( nullptr )
{
}

TMap<FViewportActionKeyInput, TArray<FKey>> UVREditorInteractor::GetKnownActionMappings(
	EControllerHand InHand /* = EControllerHand::AnyHand */,
	FName InHMDDeviceType /* = NAME_None */
) const
{
	TMap<FViewportActionKeyInput, TArray<FKey>> ReturnMap;

	if (InHand == EControllerHand::Left || InHand == EControllerHand::AnyHand)
	{
		if (InHMDDeviceType == SteamVRDeviceType || InHMDDeviceType == OpenXRDeviceType || InHMDDeviceType == NAME_None)
		{
			// HTC Vive
			ReturnMap.FindOrAdd(FViewportActionKeyInput(ViewportWorldActionTypes::WorldMovement)).Add(EKeys::Vive_Left_Grip_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier)).Add(EKeys::Vive_Left_Menu_Click);

			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadDown)).Add(EKeys::Vive_Left_Trackpad_Down); // down
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadLeft)).Add(EKeys::Vive_Left_Trackpad_Left);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadRight)).Add(EKeys::Vive_Left_Trackpad_Right);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadUp)).Add(EKeys::Vive_Left_Trackpad_Up);

			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TriggerAxis)).Add(EKeys::Vive_Left_Trigger_Axis);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionX)).Add(EKeys::Vive_Left_Trackpad_X);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionY)).Add(EKeys::Vive_Left_Trackpad_Y);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::ConfirmRadialSelection)).Add(EKeys::Vive_Left_Trackpad_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Touch)).Add(EKeys::Vive_Left_Trackpad_Touch);

			// Valve Index
			ReturnMap.FindOrAdd(FViewportActionKeyInput(ViewportWorldActionTypes::WorldMovement)).Add(EKeys::ValveIndex_Left_Trackpad_Touch);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier)).Add(EKeys::ValveIndex_Left_A_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier2)).Add(EKeys::ValveIndex_Left_B_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Touch)).Add(EKeys::ValveIndex_Left_Thumbstick_Touch);

			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadDown)).Add(EKeys::ValveIndex_Left_Thumbstick_Down); // down
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadUp)).Add(EKeys::ValveIndex_Left_Thumbstick_Up);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadLeft)).Add(EKeys::ValveIndex_Left_Thumbstick_Left);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadRight)).Add(EKeys::ValveIndex_Left_Thumbstick_Right);

			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TriggerAxis)).Add(EKeys::ValveIndex_Left_Trigger_Axis);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionX)).Add(EKeys::ValveIndex_Left_Thumbstick_X);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionY)).Add(EKeys::ValveIndex_Left_Thumbstick_Y);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::ConfirmRadialSelection)).Add(EKeys::ValveIndex_Left_Thumbstick_Click);
		}

		if (InHMDDeviceType == OculusDeviceType || InHMDDeviceType == OpenXRDeviceType || InHMDDeviceType == NAME_None)
		{
			ReturnMap.FindOrAdd(FViewportActionKeyInput(ViewportWorldActionTypes::WorldMovement)).Add(EKeys::OculusTouch_Left_Grip_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier)).Add(EKeys::OculusTouch_Left_X_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier2)).Add(EKeys::OculusTouch_Left_Y_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Touch)).Add(EKeys::OculusTouch_Left_Thumbstick_Touch);

			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadDown)).Add(EKeys::OculusTouch_Left_Thumbstick_Down); // down
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadUp)).Add(EKeys::OculusTouch_Left_Thumbstick_Up);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadLeft)).Add(EKeys::OculusTouch_Left_Thumbstick_Left);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadRight)).Add(EKeys::OculusTouch_Left_Thumbstick_Right);

			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TriggerAxis)).Add(EKeys::OculusTouch_Left_Trigger_Axis);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionX)).Add(EKeys::OculusTouch_Left_Thumbstick_X);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionY)).Add(EKeys::OculusTouch_Left_Thumbstick_Y);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::ConfirmRadialSelection)).Add(EKeys::OculusTouch_Left_Thumbstick_Click);
		}
	}

	if (InHand == EControllerHand::Right || InHand == EControllerHand::AnyHand)
	{
		if (InHMDDeviceType == SteamVRDeviceType || InHMDDeviceType == OpenXRDeviceType || InHMDDeviceType == NAME_None)
		{
			// HTC Vive
			ReturnMap.FindOrAdd(FViewportActionKeyInput(ViewportWorldActionTypes::WorldMovement)).Add(EKeys::Vive_Right_Grip_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier)).Add(EKeys::Vive_Right_Menu_Click);

			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadDown)).Add(EKeys::Vive_Right_Trackpad_Down); // down
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadLeft)).Add(EKeys::Vive_Right_Trackpad_Left);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadRight)).Add(EKeys::Vive_Right_Trackpad_Right);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadUp)).Add(EKeys::Vive_Right_Trackpad_Up);

			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TriggerAxis)).Add(EKeys::Vive_Right_Trigger_Axis);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionX)).Add(EKeys::Vive_Right_Trackpad_X);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionY)).Add(EKeys::Vive_Right_Trackpad_Y);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::ConfirmRadialSelection)).Add(EKeys::Vive_Right_Trackpad_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Touch)).Add(EKeys::Vive_Right_Trackpad_Touch);

			// Valve Index
			ReturnMap.FindOrAdd(FViewportActionKeyInput(ViewportWorldActionTypes::WorldMovement)).Add(EKeys::ValveIndex_Right_Trackpad_Touch);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier)).Add(EKeys::ValveIndex_Right_A_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier2)).Add(EKeys::ValveIndex_Right_B_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Touch)).Add(EKeys::ValveIndex_Right_Thumbstick_Touch);

			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadDown)).Add(EKeys::ValveIndex_Right_Thumbstick_Down); // down
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadUp)).Add(EKeys::ValveIndex_Right_Thumbstick_Up);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadLeft)).Add(EKeys::ValveIndex_Right_Thumbstick_Left);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadRight)).Add(EKeys::ValveIndex_Right_Thumbstick_Right);

			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TriggerAxis)).Add(EKeys::ValveIndex_Right_Trigger_Axis);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionX)).Add(EKeys::ValveIndex_Right_Thumbstick_X);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionY)).Add(EKeys::ValveIndex_Right_Thumbstick_Y);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::ConfirmRadialSelection)).Add(EKeys::ValveIndex_Right_Thumbstick_Click);
		}

		if (InHMDDeviceType == OculusDeviceType || InHMDDeviceType == OpenXRDeviceType || InHMDDeviceType == NAME_None)
		{
			ReturnMap.FindOrAdd(FViewportActionKeyInput(ViewportWorldActionTypes::WorldMovement)).Add(EKeys::OculusTouch_Right_Grip_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier)).Add(EKeys::OculusTouch_Right_A_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Modifier2)).Add(EKeys::OculusTouch_Right_B_Click);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::Touch)).Add(EKeys::OculusTouch_Right_Thumbstick_Touch);

			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadDown)).Add(EKeys::OculusTouch_Right_Thumbstick_Down); // down
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadUp)).Add(EKeys::OculusTouch_Right_Thumbstick_Up);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadLeft)).Add(EKeys::OculusTouch_Right_Thumbstick_Left);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::TrackpadRight)).Add(EKeys::OculusTouch_Right_Thumbstick_Right);

			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TriggerAxis)).Add(EKeys::OculusTouch_Right_Trigger_Axis);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionX)).Add(EKeys::OculusTouch_Right_Thumbstick_X);
			ReturnMap.FindOrAdd(FViewportActionKeyInput::Axis(UVREditorInteractor::TrackpadPositionY)).Add(EKeys::OculusTouch_Right_Thumbstick_Y);
			ReturnMap.FindOrAdd(FViewportActionKeyInput(VRActionTypes::ConfirmRadialSelection)).Add(EKeys::OculusTouch_Right_Thumbstick_Click);
		}
	}

	return ReturnMap;
}

void UVREditorInteractor::Init_Implementation( UVREditorMode* InVRMode )
{
	VRMode = InVRMode;
	KeyToActionMap.Reset();

	const FName HMDDeviceType = InVRMode->GetHMDDeviceType();

	// Setup keys
	if (ControllerMotionSource == IMotionController::LeftHandSourceId)
	{
		AddKeyAction( UVREditorInteractor::MotionController_Left_FullyPressedTriggerAxis, FViewportActionKeyInput( ViewportWorldActionTypes::SelectAndMove_FullyPressed ) );
		AddKeyAction( UVREditorInteractor::MotionController_Left_PressedTriggerAxis, FViewportActionKeyInput( ViewportWorldActionTypes::SelectAndMove ) );

		for (const TPair<FViewportActionKeyInput, TArray<FKey>>& ActionMappings : GetKnownActionMappings(EControllerHand::Left, HMDDeviceType))
		{
			for (const FKey& Key : ActionMappings.Value)
			{
				AddKeyAction(Key, ActionMappings.Key);
			}
		}
	}
	else if (ControllerMotionSource == IMotionController::RightHandSourceId)
	{
		AddKeyAction( UVREditorInteractor::MotionController_Right_FullyPressedTriggerAxis, FViewportActionKeyInput( ViewportWorldActionTypes::SelectAndMove_FullyPressed ) );
		AddKeyAction( UVREditorInteractor::MotionController_Right_PressedTriggerAxis, FViewportActionKeyInput( ViewportWorldActionTypes::SelectAndMove ) );

		for (const TPair<FViewportActionKeyInput, TArray<FKey>>& ActionMappings : GetKnownActionMappings(EControllerHand::Right, HMDDeviceType))
		{
			for (const FKey& Key : ActionMappings.Value)
			{
				AddKeyAction(Key, ActionMappings.Key);
			}
		}
	}

	bHaveMotionController = true;
}


void UVREditorInteractor::SetupComponent_Implementation( AActor* OwningActor )
{
	OwningAvatar = OwningActor;

	// Setup a motion controller component.  This allows us to take advantage of late frame updates, so
	// our motion controllers won't lag behind the HMD
	{
		MotionControllerComponent = NewObject<UMotionControllerComponent>(OwningAvatar);
		OwningAvatar->AddOwnedComponent( MotionControllerComponent );
		MotionControllerComponent->SetupAttachment( OwningAvatar->GetRootComponent() );
		MotionControllerComponent->RegisterComponent();
		MotionControllerComponent->SetMobility( EComponentMobility::Movable );
		MotionControllerComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );
		MotionControllerComponent->MotionSource = ControllerMotionSource;
	}

	const UVREditorAssetContainer& AssetContainer = VRMode->GetAssetContainer();

	// Hand mesh
	{
		HandMeshComponent = VRMode->CreateMotionControllerMesh(OwningAvatar, MotionControllerComponent);
		if (ControllerMotionSource == IMotionController::RightHandSourceId &&
			GetHMDDeviceType() == OculusDeviceType)	// Oculus has asymmetrical controllers, so we mirror the mesh horizontally
		{
			HandMeshBaseScale = FVector3d(1.0, -1.0, 1.0);
		}
		else
		{
			HandMeshBaseScale = FVector3d(1.0, 1.0, 1.0);
		}
		SetHandMeshComponentProperties();

		UMaterialInterface* HandMeshMaterial = GetVRMode().GetHMDDeviceType() == SteamVRDeviceType ? AssetContainer.VivePreControllerMaterial : AssetContainer.OculusControllerMaterial;
		check( HandMeshMaterial != nullptr );
		HandMeshMID = UMaterialInstanceDynamic::Create( HandMeshMaterial, GetTransientPackage() );
		check( HandMeshMID != nullptr );
		HandMeshComponent->SetMaterial( 0, HandMeshMID );
	}

	if (GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetSystemName() == OpenXRDeviceType)
	{
		LaserMotionControllerComponent = NewObject<UMotionControllerComponent>(OwningAvatar);
		OwningAvatar->AddOwnedComponent(LaserMotionControllerComponent);
		LaserMotionControllerComponent->SetupAttachment(OwningAvatar->GetRootComponent());
		LaserMotionControllerComponent->RegisterComponent();
		LaserMotionControllerComponent->SetMobility(EComponentMobility::Movable);
		LaserMotionControllerComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		LaserMotionControllerComponent->MotionSource =
			(ControllerMotionSource == IMotionController::LeftHandSourceId)
			? FName("LeftAim")
			: FName("RightAim");
	}

	{
		UMaterialInterface* LaserPointerMaterial = AssetContainer.LaserPointerMaterial;
		check( LaserPointerMaterial != nullptr );
		LaserPointerMID = UMaterialInstanceDynamic::Create( LaserPointerMaterial, GetTransientPackage() );
		check( LaserPointerMID != nullptr );

		UMaterialInterface* TranslucentLaserPointerMaterial = AssetContainer.LaserPointerTranslucentMaterial;
		check( TranslucentLaserPointerMaterial != nullptr );
		TranslucentLaserPointerMID = UMaterialInstanceDynamic::Create( TranslucentLaserPointerMaterial, GetTransientPackage() );
		check( TranslucentLaserPointerMID != nullptr );
	}

	// Hover cue for laser pointer
	{
		HoverMeshComponent = NewObject<UStaticMeshComponent>(OwningAvatar);
		OwningAvatar->AddOwnedComponent( HoverMeshComponent );
		HoverMeshComponent->SetupAttachment(OwningAvatar->GetRootComponent() );
		HoverMeshComponent->RegisterComponent();

		UStaticMesh* HoverMesh = AssetContainer.LaserPointerHoverMesh;
		check( HoverMesh != nullptr );
		HoverMeshComponent->SetStaticMesh( HoverMesh );
		HoverMeshComponent->SetMobility( EComponentMobility::Movable );
		HoverMeshComponent->SetCollisionEnabled( ECollisionEnabled::NoCollision );
		HoverMeshComponent->SetCastShadow( false );

		HoverMeshComponent->SetMaterial( 0, LaserPointerMID );
		HoverMeshComponent->SetMaterial( 1, TranslucentLaserPointerMID );

		// Add a light!
		{
			HoverPointLightComponent = NewObject<UPointLightComponent>(OwningAvatar);
			OwningAvatar->AddOwnedComponent( HoverPointLightComponent );
			HoverPointLightComponent->SetupAttachment( HoverMeshComponent );
			HoverPointLightComponent->RegisterComponent();

			HoverPointLightComponent->SetLightColor( FLinearColor::Red );
			//Hand.HoverPointLightComponent->SetLightColor( FLinearColor( 0.0f, 1.0f, 0.2f, 1.0f ) );
			HoverPointLightComponent->SetIntensity( 30.0f );	// @todo: VREditor tweak
			HoverPointLightComponent->SetMobility( EComponentMobility::Movable );
			HoverPointLightComponent->SetAttenuationRadius( VREd::LaserPointerLightRadius->GetFloat() );
			HoverPointLightComponent->bUseInverseSquaredFalloff = false;
			HoverPointLightComponent->SetCastShadows( false );
		}
	}

	{
		const int32 NumLaserSplinePoints = 12;

		UStaticMesh* MiddleSplineMesh = AssetContainer.LaserPointerMesh;
		check( MiddleSplineMesh != nullptr );
		UStaticMesh* StartSplineMesh = AssetContainer.LaserPointerStartMesh;
		check( StartSplineMesh != nullptr );
		UStaticMesh* EndSplineMesh = AssetContainer.LaserPointerEndMesh;
		check( EndSplineMesh != nullptr );

		LaserSplineComponent = NewObject<USplineComponent>(OwningAvatar);
		OwningAvatar->AddOwnedComponent( LaserSplineComponent );
		LaserSplineComponent->SetupAttachment( LaserMotionControllerComponent ? LaserMotionControllerComponent : MotionControllerComponent );
		LaserSplineComponent->RegisterComponent();
		LaserSplineComponent->SetVisibility( false );
		LaserSplineMeshComponents.Empty();

		for (int32 i = 0; i < NumLaserSplinePoints; i++)
		{
			USplineMeshComponent* SplineSegment = NewObject<USplineMeshComponent>(OwningAvatar);
			SplineSegment->SetMobility( EComponentMobility::Movable );
			SplineSegment->SetCollisionEnabled( ECollisionEnabled::NoCollision );
			SplineSegment->SetbNeverNeedsCookedCollisionData( true );
			SplineSegment->SetSplineUpDir( FVector::UpVector, false );

			UStaticMesh* StaticMesh = nullptr;
			if (i == 0)
			{
				StaticMesh = StartSplineMesh;
			}
			else if (i == NumLaserSplinePoints - 1)
			{
				StaticMesh = EndSplineMesh;
			}
			else
			{
				StaticMesh = MiddleSplineMesh;
			}

			SplineSegment->SetStaticMesh( StaticMesh );
			SplineSegment->bTickInEditor = true;
			SplineSegment->bCastDynamicShadow = false;
			SplineSegment->CastShadow = false;
			SplineSegment->SetMaterial( 0, LaserPointerMID );
			SplineSegment->SetMaterial( 1, TranslucentLaserPointerMID );
			SplineSegment->SetVisibility( true );
			SplineSegment->RegisterComponent();

			LaserSplineMeshComponents.Add( SplineSegment );
		}
	}
}


void UVREditorInteractor::ReplaceHandMeshComponent(
	UStaticMesh* NewMesh,
	FVector MeshScale /* = FVector(1.f, 1.f, 1.f) */
)
{
	HandMeshComponent->UnregisterComponent();
	HandMeshComponent->DestroyComponent();
	HandMeshComponent = VRMode->CreateMotionControllerMesh(OwningAvatar, MotionControllerComponent, NewMesh);
	HandMeshBaseScale = MeshScale;
	SetHandMeshComponentProperties();
}

void UVREditorInteractor::SetHandMeshComponentProperties()
{
	check(HandMeshComponent != nullptr);

	HandMeshComponent->SetCastShadow(false);
	HandMeshComponent->SetCollisionEnabled(ECollisionEnabled::PhysicsOnly);
	HandMeshComponent->SetCollisionResponseToAllChannels(ECR_Overlap);
	HandMeshComponent->SetGenerateOverlapEvents(true);

	HandMeshGripTransform = FTransform::Identity;

	// To help ease the transition to OpenXR, an alternative controller mesh origin can be specified.
	if (GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetSystemName() == OpenXRDeviceType)
	{
		const FName GripSocket("OpenXrGrip");
		if (HandMeshComponent->DoesSocketExist(GripSocket))
		{
			HandMeshGripTransform = HandMeshComponent->GetSocketTransform(GripSocket, RTS_Component);
		}
	}

	UpdateHandMeshRelativeTransform();
}

void UVREditorInteractor::UpdateHandMeshRelativeTransform_Implementation()
{
	// The hands need to stay the same size relative to our tracking space, so we inverse compensate for world to meters scale here
	const float WorldScaleFactor = WorldInteraction->GetWorldScaleFactor();
	const FTransform ScaleTransform(FRotator::ZeroRotator, FVector::ZeroVector, HandMeshBaseScale / WorldScaleFactor);
	const FTransform NewRelativeTransform = (ScaleTransform * HandMeshGripTransform).Inverse();
	HandMeshComponent->SetRelativeTransform(NewRelativeTransform);
}

void UVREditorInteractor::Shutdown_Implementation()
{
	VRMode = nullptr;
	MotionControllerComponent = nullptr;
	HandMeshComponent = nullptr;
	LaserPointerMID = nullptr;
	TranslucentLaserPointerMID = nullptr;
	HoverMeshComponent = nullptr;
	HoverPointLightComponent = nullptr;
	HandMeshComponent = nullptr;

	for (auto& KeyAndValue : HelpLabels)
	{
		AFloatingText* FloatingText = KeyAndValue.Value;
		GetVRMode().DestroyTransientActor( FloatingText );
	}

	HelpLabels.Empty();

	VRMode = nullptr;

	Super::Shutdown_Implementation();
}

EControllerHand UVREditorInteractor::GetControllerSide() const
{
	EControllerHand Hand = EControllerHand::Left;
	IMotionController::GetHandEnumForSourceName( ControllerMotionSource, Hand );
	return Hand;

}

EControllerType UVREditorInteractor::GetControllerType() const
{
	return OverrideControllerType != EControllerType::Unknown ? OverrideControllerType : ControllerType;
}

void UVREditorInteractor::SetControllerType( const EControllerType InControllerType )
{
	OverrideControllerType = EControllerType::Unknown;
	ControllerType = InControllerType;
}

bool UVREditorInteractor::TryOverrideControllerType(const EControllerType InControllerType)
{
	if (InControllerType != EControllerType::Unknown && OverrideControllerType != EControllerType::Unknown)
	{
		return false;
	}

	OverrideControllerType = InControllerType;
	return true;
}

void UVREditorInteractor::Tick_Implementation( const float DeltaTime )
{
	Super::Tick_Implementation( DeltaTime );

	// @todo vreditor: Manually ticking motion controller components
	MotionControllerComponent->TickComponent( DeltaTime, ELevelTick::LEVELTICK_PauseTick, nullptr );
	LaserMotionControllerComponent->TickComponent( DeltaTime, ELevelTick::LEVELTICK_PauseTick, nullptr );

	UpdateHandMeshRelativeTransform();

	UpdateRadialMenuInput( DeltaTime );

	{
		const float WorldScaleFactor = WorldInteraction->GetWorldScaleFactor();

		// Don't bother drawing hands if we're not currently tracking them.
		if (bHaveMotionController)
		{
			HandMeshComponent->SetVisibility( true );
		}
		else
		{
			HandMeshComponent->SetVisibility( false );
		}

		// The laser pointer needs to stay the same size relative to our tracking space, so we inverse compensate for world to meters scale here
		float LaserPointerRadius = VREd::LaserPointerRadius->GetFloat() * WorldScaleFactor;
		float HoverMeshRadius = VREd::LaserPointerHoverBallRadius->GetFloat() * WorldScaleFactor;

		// If we're hovering over something really close to the camera, go ahead and shrink the effect
		// @todo vreditor: Can we make this actually just sized based on distance automatically?  The beam and impact point are basically a cone.
		if (IsHoveringOverUI())
		{
			LaserPointerRadius *= VREd::LaserRadiusScaleWhenOverUI->GetFloat();
			HoverMeshRadius *= VREd::HoverBallRadiusScaleWhenOverUI->GetFloat();
		}

		const bool bEvenIfBlocked = false;

		// If we're currently grabbing the world with this interactor
		const bool bDraggingWorld = (InteractorData.DraggingMode == EViewportInteractionDraggingMode::World ||
			(GetOtherInteractor() != nullptr && GetOtherInteractor()->GetInteractorData().DraggingMode == EViewportInteractionDraggingMode::World && InteractorData.DraggingMode == EViewportInteractionDraggingMode::AssistingDrag));

		FVector LaserPointerStart(0), LaserPointerEnd(0);
		const bool bHasLaser = GetLaserPointer(/* Out */ LaserPointerStart, /* Out */ LaserPointerEnd, bEvenIfBlocked );
		if (bForceShowLaser || (bHasLaser && !bDraggingWorld))
		{
			// Only show the laser if we're actually in VR
			SetLaserVisibility( GetVRMode().IsActuallyUsingVR() );

			// NOTE: We don't need to set the laser pointer location and rotation, as the MotionControllerComponent will do
			// that later in the frame.

			// If we're actively dragging something around, then we'll crop the laser length to the hover impact
			// point.  Otherwise, we always want the laser to protrude through hovered objects, so that you can
			// interact with translucent gizmo handles that are occluded by geometry
			if (IsHoveringOverGizmo() ||
				IsHoveringOverUI() ||
				IsHovering())
			{
				LaserPointerEnd = GetHoverLocation();
			}

			if (IsHovering() && !GetIsLaserBlocked())
			{
				const FVector DirectionTowardHoverLocation = (GetHoverLocation() - LaserPointerStart).GetSafeNormal();

				// The hover effect needs to stay the same size relative to our tracking space, so we inverse compensate for world to meters scale here
				HoverMeshComponent->SetRelativeScale3D( FVector( HoverMeshRadius * 2.0f ) * (0.25f + 1.0f - this->GetSelectAndMoveTriggerValue() * 0.75f) );
				HoverMeshComponent->SetVisibility( true );
				HoverMeshComponent->SetWorldLocation( GetHoverLocation() );

				// Show the light too, unless it's on top of UI.  It looks too distracting on top of UI.
				HoverPointLightComponent->SetVisibility( !IsHoveringOverUI() );

				// Update radius for world scaling
				HoverPointLightComponent->SetAttenuationRadius( VREd::LaserPointerLightRadius->GetFloat() * WorldScaleFactor );

				// Pull hover light back a bit from the end of the ray
				const float PullBackAmount = VREd::LaserPointerLightPullBackDistance->GetFloat() * WorldInteraction->GetWorldScaleFactor();
				HoverPointLightComponent->SetWorldLocation( GetHoverLocation() - PullBackAmount * DirectionTowardHoverLocation );
			}
			else
			{
				HoverMeshComponent->SetVisibility( false );
				HoverPointLightComponent->SetVisibility( false );
			}
		}
		else
		{
			SetLaserVisibility( false );
			HoverMeshComponent->SetVisibility( false );
			HoverPointLightComponent->SetVisibility( false );
		}

		// Update the curved laser. No matter if we actually show the laser it needs to update,
		// so if in the next frame it needs to be visible it won't interpolate from a previous location.
		{
			// Offset the beginning of the laser pointer a bit, so that it doesn't overlap the hand mesh
			float LaserPointerStartOffset = WorldScaleFactor *
				(GetVRMode().GetHMDDeviceType() == OculusDeviceType
					? VREd::OculusLaserPointerStartOffset->GetFloat()
					: VREd::ViveLaserPointerStartOffset->GetFloat());

			if (LaserMotionControllerComponent)
			{
				LaserPointerStartOffset = 0.0f;
			}

			// Get the hand transform and forward vector.
			FTransform InteractorTransform;
			FVector InteractorForwardVector;

			if (GetTransformAndForwardVector( /* Out */ InteractorTransform, /* Out */ InteractorForwardVector))
			{
				InteractorForwardVector.Normalize();

				// Offset the start point of the laser.
				LaserPointerStart = InteractorTransform.GetLocation() + (InteractorForwardVector * LaserPointerStartOffset);

				UpdateSplineLaser(LaserPointerStart, LaserPointerEnd, InteractorForwardVector);
			}
		}

		bForceShowLaser = false;
	}

	// Updating laser colors for both hands
	{
		FLinearColor ResultColor;
		float CrawlSpeed = 0.0f;
		float CrawlFade = 0.0f;

		if (ForceLaserColor.IsSet())
		{
			ResultColor = ForceLaserColor.GetValue();
			ForceLaserColor.Reset();
		}
		else
		{
			if (InteractorData.HoveringOverTransformGizmoComponent != nullptr)
			{
				ResultColor = WorldInteraction->GetColor( UViewportWorldInteraction::EColors::GizmoHover );
			}
			else
			{
				const EViewportInteractionDraggingMode DraggingMode = GetDraggingMode();
				if (DraggingMode == EViewportInteractionDraggingMode::World ||
					(DraggingMode == EViewportInteractionDraggingMode::AssistingDrag && GetOtherInteractor() != nullptr && GetOtherInteractor()->GetDraggingMode() == EViewportInteractionDraggingMode::World))
				{
					// We can teleport in this mode, so animate the laser a bit
					CrawlFade = 1.0f;
					CrawlSpeed = 5.0f;
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::WorldDraggingColor );
				}
				else if (DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact ||
					DraggingMode == EViewportInteractionDraggingMode::Material ||
					DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely ||
					DraggingMode == EViewportInteractionDraggingMode::AssistingDrag)
				{
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::SelectionColor );
				}
				else if (DraggingMode == EViewportInteractionDraggingMode::TransformablesWithGizmo)
				{
					ResultColor = WorldInteraction->GetColor( UViewportWorldInteraction::EColors::GizmoHover );
				}
				else if (DraggingMode == EViewportInteractionDraggingMode::Interactable ||
					(GetVRMode().GetUISystem().IsInteractorDraggingDockUI( this ) && GetVRMode().GetUISystem().IsDraggingDockUI()))
				{
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::UIColor );
				}
				else if (GetLastHoverComponent() != nullptr && GetLastHoverComponent()->GetOwner() != nullptr && GetLastHoverComponent()->GetOwner()->IsA( AVREditorDockableWindow::StaticClass() ))
				{
					AVREditorDockableWindow* HoveredDockWindow = Cast<AVREditorDockableWindow>( GetLastHoverComponent()->GetOwner() );
					if (HoveredDockWindow && HoveredDockWindow->GetSelectionBarMeshComponent() == GetLastHoverComponent())
					{
						ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::UIColor );
					}
					else
					{
						ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::DefaultColor );
					}
				}
				else if (GetControllerType() == EControllerType::Laser && VRMode->IsAimingTeleport())
				{
					CrawlFade = 1.0f;
					CrawlSpeed = 5.0f;
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::WorldDraggingColor );
				}
				else if (GetControllerType() == EControllerType::UI)
				{
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::UIColor );
				}
				else if (IsHoveringOverSelectedActor())
				{
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::SelectionColor );
				}
				else
				{
					ResultColor = GetVRMode().GetColor( UVREditorMode::EColors::DefaultColor );
				}
			}
		}

		SetLaserVisuals( ResultColor, CrawlFade, CrawlSpeed );
	}

	UpdateHelpLabels();

	// If the other controller is dragging freely, the UI controller can assist
	if (GetControllerType() == EControllerType::UI)
	{
		if (GetOtherInteractor() &&
			(GetOtherInteractor()->GetDraggingMode() == EViewportInteractionDraggingMode::TransformablesFreely))
		{
			TryOverrideControllerType( EControllerType::AssistingLaser );
		}
	}
	// Otherwise the UI controller resets to a UI controller
	// Allow for "trading off" during an assisted drag
	else if (GetControllerType() == EControllerType::AssistingLaser)
	{
		if (GetOtherInteractor() &&
			!(GetOtherInteractor()->GetDraggingMode() == EViewportInteractionDraggingMode::TransformablesFreely ||
				GetOtherInteractor()->GetInteractorData().bWasAssistingDrag))
		{
			TryOverrideControllerType( EControllerType::Unknown );
		}
	}
}

FName UVREditorInteractor::GetHMDDeviceType() const
{
	return GetVRMode().GetHMDDeviceType();
}

void UVREditorInteractor::CalculateDragRay( double& InOutDragRayLength, double& InOutDragRayVelocity )
{
	const FTimespan CurrentTime = FTimespan::FromSeconds( FPlatformTime::Seconds() );
	const float WorldScaleFactor = WorldInteraction->GetWorldScaleFactor();

	// If we're dragging an object, go ahead and slide the object along the ray based on how far they slide their touch
	// Make sure they are touching the trackpad, otherwise we get bad data
	if (bIsTrackpadPositionValid[1])
	{
		const bool bIsAbsolute = (GetVRMode().GetHMDDeviceType() == SteamVRDeviceType);
		float SlideDelta = GetTrackpadSlideDelta() * WorldScaleFactor;

		if (!FMath::IsNearlyZero( SlideDelta ))
		{
			InOutDragRayLength += SlideDelta;

			InOutDragRayVelocity = 0.0f;

			// Don't apply inertia unless the user dragged a decent amount this frame
			if (bIsAbsolute && FMath::Abs( SlideDelta ) >= VREd::MinVelocityForInertia->GetFloat() * WorldScaleFactor)
			{
				// Don't apply inertia if our data is sort of old
				if (CurrentTime - LastTrackpadPositionUpdateTime <= FTimespan::FromSeconds( 1.0f / 30.0f ))
				{
					InOutDragRayVelocity = SlideDelta;
				}
			}

			// Don't go too far
			if (InOutDragRayLength < 0.0f)
			{
				InOutDragRayLength = 0.0f;
				InOutDragRayVelocity = 0.0f;
			}

			// Stop transforming object to laser impact point when trying to slide with touchpad or analog stick.
			if (InteractorData.DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact && !FMath::IsNearlyZero( SlideDelta, VREd::TrackpadStopImpactAtLaserBuffer->GetFloat() ))
			{
				InteractorData.DraggingMode = EViewportInteractionDraggingMode::TransformablesFreely;
			}
		}
	}
	else
	{
		if (!FMath::IsNearlyZero( InOutDragRayVelocity ))
		{
			// Apply drag ray length inertia
			InOutDragRayLength += InOutDragRayVelocity;

			// Don't go too far!
			if (InOutDragRayLength < 0.0f)
			{
				InOutDragRayLength = 0.0f;
				InOutDragRayVelocity = 0.0f;
			}

			// Apply damping
			FVector RayVelocityVector( InOutDragRayVelocity, 0.0f, 0.0f );
			const bool bVelocitySensitive = true;
			WorldInteraction->ApplyVelocityDamping( RayVelocityVector, bVelocitySensitive );
			InOutDragRayVelocity = RayVelocityVector.X;
		}
		else
		{
			InOutDragRayVelocity = 0.0f;
		}
	}
}

FHitResult UVREditorInteractor::GetHitResultFromLaserPointer( TArray<AActor*>* OptionalListOfIgnoredActors /*= nullptr*/, const EHitResultGizmoFilterMode GizmoFilterMode /*= false*/,
	TArray<UClass*>* ObjectsInFrontOfGizmo /*= nullptr */, const bool bEvenIfBlocked /*= false */, const float LaserLengthOverride /*= 0.0f */ )
{
	static TArray<AActor*> IgnoredActors;
	IgnoredActors.Reset();
	if (OptionalListOfIgnoredActors == nullptr)
	{
		OptionalListOfIgnoredActors = &IgnoredActors;
	}

	// Ignore UI widgets too
	if (GetDraggingMode() == EViewportInteractionDraggingMode::TransformablesAtLaserImpact)
	{
		for (TActorIterator<AVREditorFloatingUI> UIActorIt( WorldInteraction->GetWorld(), AVREditorFloatingUI::StaticClass() ); UIActorIt; ++UIActorIt)
		{
			OptionalListOfIgnoredActors->Add( *UIActorIt );
		}
	}

	static TArray<UClass*> PriorityOverGizmoObjects;
	PriorityOverGizmoObjects.Reset();
	if (ObjectsInFrontOfGizmo == nullptr)
	{
		ObjectsInFrontOfGizmo = &PriorityOverGizmoObjects;
	}

	ObjectsInFrontOfGizmo->Add( AVREditorDockableWindow::StaticClass() );
	ObjectsInFrontOfGizmo->Add( AVREditorFloatingUI::StaticClass() );

	return UViewportInteractor::GetHitResultFromLaserPointer( OptionalListOfIgnoredActors, GizmoFilterMode, ObjectsInFrontOfGizmo, bEvenIfBlocked, LaserLengthOverride );
}

void UVREditorInteractor::PreviewInputKey( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled )
{
	if (bIsScrubbingSequence &&
		GetControllerType() == EControllerType::UI &&
		Action.ActionType == ViewportWorldActionTypes::SelectAndMove &&
		Action.Event == IE_Pressed)
	{
		ToggleSequencerScrubbingMode();
		GetVRMode().GetUISystem().TryToSpawnRadialMenu( this, true );
	}

	// Update touch state
	if (Action.ActionType == VRActionTypes::Touch)
	{
		if (Event == IE_Pressed)
		{
			// Set initial position when starting to touch the trackpad
			InitialTouchPosition = TrackpadPosition;
		}
		else if (Event == IE_Released)
		{
			bIsTrackpadPositionValid[0] = false;
			bIsTrackpadPositionValid[1] = false;

			// Detect swipe on trackpad.
			const FVector2D SwipeDelta = LastTrackpadPosition - InitialTouchPosition;
			const float AbsSwipeDeltaX = FMath::Abs( (float)SwipeDelta.X );
			const float AbsSwipeDeltaY = FMath::Abs( (float)SwipeDelta.Y );
			if (!FMath::IsNearlyZero( SwipeDelta.X, 1.0f ) && AbsSwipeDeltaX > AbsSwipeDeltaY)
			{
				if (SwipeDelta.X > 0)
				{
					LastSwipe = ETouchSwipeDirection::Right;
					if (bIsUndoRedoSwipeEnabled)
					{
						UndoRedoFromSwipe( LastSwipe );
					}
				}
				else if (SwipeDelta.X < 0)
				{
					LastSwipe = ETouchSwipeDirection::Left;
					if (bIsUndoRedoSwipeEnabled)
					{
						UndoRedoFromSwipe( LastSwipe );
					}
				}
			}
			else if (!FMath::IsNearlyZero( SwipeDelta.Y, 1.0f ))
			{
				if (SwipeDelta.Y > 0)
				{
					LastSwipe = ETouchSwipeDirection::Up;
				}
				else if (SwipeDelta.Y < 0)
				{
					LastSwipe = ETouchSwipeDirection::Down;
				}
			}
		}
	}

	ActionKeysPressed.FindOrAdd(Action.ActionType) = (Event == IE_Released) ? false : true;

	if (!bOutWasHandled)
	{
		Super::PreviewInputKey( ViewportClient, Action, Key, Event, bOutWasHandled );
	}
}

void UVREditorInteractor::HandleInputKey( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled )
{
	if (!bOutWasHandled && Action.ActionType == VRActionTypes::ConfirmRadialSelection)
	{
		bOutWasHandled = true;
		const EViewportInteractionDraggingMode DraggingMode = GetDraggingMode();

		if (Event == IE_Pressed)
		{
			// Start dragging at laser impact when already dragging actors freely
			if (!IsCarrying() && DraggingMode == EViewportInteractionDraggingMode::TransformablesFreely)
			{
				FVector PlaceAt = GetHoverLocation();
				const bool bIsPlacingActors = true;
				const bool bAllowInterpolationWhenPlacing = true;
				const bool bShouldUseLaserImpactDrag = true;
				const bool bStartTransaction = true;
				const bool bWithGrabberSphere = false;	// Never use the grabber sphere when dragging at laser impact
				WorldInteraction->StartDragging( this, WorldInteraction->GetTransformGizmoActor()->GetRootComponent(), PlaceAt, bIsPlacingActors, bAllowInterpolationWhenPlacing, bShouldUseLaserImpactDrag, bStartTransaction, bWithGrabberSphere );
			}
		}
		else if (Event == IE_Released)
		{
			// Disable dragging at laser impact when releasing
			if (DraggingMode == EViewportInteractionDraggingMode::TransformablesAtLaserImpact)
			{
				SetDraggingMode( EViewportInteractionDraggingMode::TransformablesFreely );
			}
		}
	}

	ApplyButtonPressColors( Action );
	Super::HandleInputKey( ViewportClient, Action, Key, Event, bOutWasHandled );
}


void UVREditorInteractor::HandleInputAxis( FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const float Delta, const float DeltaTime, bool& bOutWasHandled )
{
	const FName HmdDeviceType = GetHMDDeviceType();

	if (!bOutWasHandled && Action.ActionType == TriggerAxis)
	{
		const float TriggerPressedThreshold = (HmdDeviceType == OculusDeviceType) ? GetDefault<UVRModeSettings>()->TriggerPressedThreshold_Rift : GetDefault<UVRModeSettings>()->TriggerPressedThreshold_Vive;
		const float TriggerDeadZone = (HmdDeviceType == OculusDeviceType) ? VREd::TriggerDeadZone_Rift->GetFloat() : VREd::TriggerDeadZone_Vive->GetFloat();

		// Synthesize "lightly pressed" events for the trigger
		{
			// Store latest trigger value amount
			SelectAndMoveTriggerValue = Delta;

			if (!bIsTriggerPressed &&			// Don't fire if we are already pressed
				bHasTriggerBeenReleasedSinceLastPress &&	// Only if we've been fully released since the last time we fired
				Delta >= TriggerPressedThreshold)
			{
				bIsTriggerPressed = true;
				bHasTriggerBeenReleasedSinceLastPress = false;
				// Synthesize an input key for this light press
				const EInputEvent InputEvent = IE_Pressed;
				const bool bWasLightPressHandled = UViewportInteractor::HandleInputKey( ViewportClient, ControllerMotionSource == IMotionController::LeftHandSourceId ? MotionController_Left_PressedTriggerAxis : MotionController_Right_PressedTriggerAxis, InputEvent );
			}
			else if (bIsTriggerPressed && Delta < TriggerPressedThreshold)
			{
				bIsTriggerPressed = false;

				// Synthesize an input key for this light press
				const EInputEvent InputEvent = IE_Released;
				const bool bWasLightReleaseHandled = UViewportInteractor::HandleInputKey( ViewportClient, ControllerMotionSource == IMotionController::LeftHandSourceId ? MotionController_Left_PressedTriggerAxis : MotionController_Right_PressedTriggerAxis, InputEvent );
			}
		}

		if (!bHasTriggerBeenReleasedSinceLastPress && Delta < TriggerDeadZone)
		{
			bHasTriggerBeenReleasedSinceLastPress = true;
		}

		// Synthesize "fully pressed" events for the trigger
		{
			const float TriggerFullyPressedThreshold = (HmdDeviceType == OculusDeviceType) ? VREd::TriggerFullyPressedThreshold_Rift->GetFloat() : VREd::TriggerFullyPressedThreshold_Vive->GetFloat();

			if (!bIsTriggerFullyPressed &&	// Don't fire if we are already pressed
				Delta >= TriggerFullyPressedThreshold)
			{
				bIsTriggerFullyPressed = true;

				const EInputEvent InputEvent = IE_Pressed;
				UViewportInteractor::HandleInputKey( ViewportClient, ControllerMotionSource == IMotionController::LeftHandSourceId ? MotionController_Left_FullyPressedTriggerAxis : MotionController_Right_FullyPressedTriggerAxis, InputEvent );
			}
			else if (bIsTriggerFullyPressed && Delta < TriggerPressedThreshold)
			{
				bIsTriggerFullyPressed = false;

				const EInputEvent InputEvent = IE_Released;
				UViewportInteractor::HandleInputKey( ViewportClient, ControllerMotionSource == IMotionController::LeftHandSourceId ? MotionController_Left_FullyPressedTriggerAxis : MotionController_Right_FullyPressedTriggerAxis, InputEvent );
			}
		}
	}

	if (!bOutWasHandled)
	{
		if (Delta != 0)
		{
			if (Action.ActionType == VRActionTypes::TrackpadPositionX)
			{
				LastTrackpadPosition.X = bIsTrackpadPositionValid[0] ? TrackpadPosition.X : Delta;
				LastTrackpadPositionUpdateTime = FTimespan::FromSeconds(FPlatformTime::Seconds());
				TrackpadPosition.X = Delta;
				bIsTrackpadPositionValid[0] = true;
			}

			if (Action.ActionType == VRActionTypes::TrackpadPositionY)
			{
				float DeltaAxis = Delta;
				if (VREd::InvertTrackpadVertical->GetInt() != 0)
				{
					DeltaAxis = -DeltaAxis;	// Y axis is inverted from HMD
				}

				LastTrackpadPosition.Y = bIsTrackpadPositionValid[1] ? TrackpadPosition.Y : DeltaAxis;
				LastTrackpadPositionUpdateTime = FTimespan::FromSeconds(FPlatformTime::Seconds());
				TrackpadPosition.Y = DeltaAxis;
				bIsTrackpadPositionValid[1] = true;
			}
		}
	}

	// Check and generate trackpad d-pad key events ourselves, which may be required if the XR runtime doesn't support them.
	if (Action.ActionType == TrackpadPositionX || Action.ActionType == TrackpadPositionY)
	{
		const bool bTrackpadValid = bIsTrackpadPositionValid[0] && bIsTrackpadPositionValid[1];
		if (VRMode->NeedsSyntheticDpad() && bTrackpadValid)
		{
			//const float WedgeAngle = 0.5f * UE_PI; // Quadrants / 4-way
			const float WedgeAngle = 3.0f * UE_PI / 4.0f; // Overlapping octants / 8-way
			const float CenterRegion = 0.75f;

			const float WedgeHalfAngle = WedgeAngle / 2.0f;

			enum EDpadDirection
			{
				Up,
				Down,
				Left,
				Right,
			};

			const float Magnitude = (float) TrackpadPosition.Size();
			const float Angle = FMath::Atan2((float)TrackpadPosition.Y, (float)TrackpadPosition.X);

			auto IsDpadDirectionPressed = [&](EDpadDirection Dir) -> bool
			{
				if (Magnitude <= CenterRegion)
				{
					return false;
				}

				switch (Dir)
				{
					case Up:    return Angle >= (UE_PI * .5f - WedgeHalfAngle)  && Angle <= (UE_PI * .5f + WedgeHalfAngle);
					case Down:  return Angle >= (-UE_PI * .5f - WedgeHalfAngle) && Angle <= (-UE_PI * .5f + WedgeHalfAngle);
					case Left:  return Angle >= (UE_PI - WedgeHalfAngle)        || Angle <= (-UE_PI + WedgeHalfAngle);
					case Right: return Angle >= (-WedgeHalfAngle)               && Angle <= (WedgeHalfAngle);
					default: checkNoEntry(); return false;
				}
			};

			auto CheckSendDpadKeyEvent = [&](EDpadDirection Dir)
			{
				FName ActionName;
				switch (Dir)
				{
					case Up:    ActionName = VRActionTypes::TrackpadUp; break;
					case Down:  ActionName = VRActionTypes::TrackpadDown; break;
					case Left:  ActionName = VRActionTypes::TrackpadLeft; break;
					case Right: ActionName = VRActionTypes::TrackpadRight; break;
					default: checkNoEntry(); return;
				}

				const bool bWasPressed = IsActionKeyPressed(ActionName);
				const bool bPressedNow = IsDpadDirectionPressed(Dir);

				if (bPressedNow != bWasPressed)
				{
					FKey InputKey;
					for (const TPair<FKey, FViewportActionKeyInput>& ActionPair : KeyToActionMap)
					{
						if (ActionPair.Value.ActionType == ActionName)
						{
							InputKey = ActionPair.Key;
						}
					}

					if (ensure(InputKey.IsValid()))
					{
						const EInputEvent InputEvent = bPressedNow ? IE_Pressed : IE_Released;
						UE_LOGF(LogVREditor, Verbose, "Synthesized D-pad event: Key %ls %ls",
							*InputKey.ToString(), InputEvent == IE_Pressed ? TEXT("pressed") : TEXT("released"));
						UViewportInteractor::HandleInputKey(ViewportClient, InputKey, InputEvent);
					}
				}
			};

			CheckSendDpadKeyEvent(Up);
			CheckSendDpadKeyEvent(Down);
			CheckSendDpadKeyEvent(Left);
			CheckSendDpadKeyEvent(Right);
		}
	}

	Super::HandleInputAxis( ViewportClient, Action, Key, Delta, DeltaTime, bOutWasHandled );
}

void UVREditorInteractor::ToggleSequencerScrubbingMode()
{
	bIsScrubbingSequence = !bIsScrubbingSequence;
}

bool UVREditorInteractor::IsScrubbingSequencer() const
{
	return bIsScrubbingSequence;
}

class UMotionControllerComponent* UVREditorInteractor::GetMotionControllerComponent() const
{
	return MotionControllerComponent;
}

void UVREditorInteractor::SetControllerHandSide( const FName InControllerHandSide )
{
	ControllerMotionSource = InControllerHandSide;
}

FName UVREditorInteractor::GetControllerHandSide() const
{
	return ControllerMotionSource;
}

void UVREditorInteractor::ResetHoverState()
{
	Super::ResetHoverState();
	bIsHoveringOverUI = false;
}

float UVREditorInteractor::GetSlideDelta_Implementation() const
{
	return GetTrackpadSlideDelta();
}

void UVREditorInteractor::PlayHapticEffect( const float Strength )
{
	IInputInterface* InputInterface = FSlateApplication::Get().GetInputInterface();
	if (InputInterface)
	{
		const double CurrentTime = FPlatformTime::Seconds();

		//@todo viewportinteration
		FForceFeedbackValues ForceFeedbackValues;
		ForceFeedbackValues.LeftLarge = ControllerMotionSource == IMotionController::LeftHandSourceId ? Strength : 0;
		ForceFeedbackValues.RightLarge = ControllerMotionSource == IMotionController::RightHandSourceId ? Strength : 0;

		// @todo vreditor: If an Xbox controller is plugged in, this causes both the motion controllers and the Xbox controller to vibrate!
		InputInterface->SetForceFeedbackChannelValues( WorldInteraction->GetMotionControllerID(), ForceFeedbackValues );
	}
}

void UVREditorInteractor::SetForceShowLaser( const bool bInForceShow )
{
	bForceShowLaser = bInForceShow;
}

bool UVREditorInteractor::IsCarrying() const
{
	const TArray<TUniquePtr<class FViewportTransformable>>& Transformables = VRMode->GetWorldInteraction().GetTransformables();
	const bool bCanBeCarried = Transformables.Num() == 1 && Transformables[0].Get()->ShouldBeCarried();
	return (bCanBeCarried && GetDraggingMode() == EViewportInteractionDraggingMode::TransformablesFreely);
}

float UVREditorInteractor::GetTrackpadSlideDelta( const bool Axis /*= 1*/ ) const
{
	if (IsCarrying())
	{
		return 0.0f;
	}

	const bool bIsAbsolute = (GetVRMode().GetHMDDeviceType() == SteamVRDeviceType);
	float SlideDelta = 0.0f;
	if (IsActionKeyPressed(VRActionTypes::Touch) || !bIsAbsolute)
	{
		if (bIsAbsolute)
		{
			SlideDelta = (float) ((TrackpadPosition[Axis] - LastTrackpadPosition[Axis]) * VREd::TrackpadAbsoluteDragSpeed->GetFloat());
		}
		else
		{
			SlideDelta = (float) (TrackpadPosition[Axis] * VREd::TrackpadRelativeDragSpeed->GetFloat());
		}
	}

	return SlideDelta;

}


bool UVREditorInteractor::IsActionKeyPressed(FName ActionName) const
{
	if (const bool* bResult = ActionKeysPressed.Find(ActionName))
	{
		return *bResult;
	}

	return false;
}


void UVREditorInteractor::PollInput()
{
	bHaveMotionController = false;
	InteractorData.LastTransform = InteractorData.Transform;
	InteractorData.LastRoomSpaceTransform = InteractorData.RoomSpaceTransform;

	// Generic motion controllers
	TArray<IMotionController*> MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>( IMotionController::GetModularFeatureName() );
	for (auto MotionController : MotionControllers)	// @todo viewportinteraction: Needs support for multiple pairs of motion controllers
	{
		if (MotionController != nullptr)
		{
			FVector Location = FVector::ZeroVector;
			FRotator Rotation = FRotator::ZeroRotator;
			float WorldScale = 100.0f;
			if (VRMode != nullptr)
			{
				WorldScale = GetVRMode().GetWorldScaleFactor() *100.0f; // WorldScaleFactor is worldscale / 100.0
			}

			const FName MotionSource = LaserMotionControllerComponent ? LaserMotionControllerComponent->MotionSource : ControllerMotionSource;
			if (MotionController->GetControllerOrientationAndPosition( WorldInteraction->GetMotionControllerID(), MotionSource,/* Out */ Rotation, /* Out */ Location, WorldScale ))
			{
				bHaveMotionController = true;
				InteractorData.RoomSpaceTransform = FTransform( Rotation.Quaternion(), Location, FVector( 1.0f ) );
				InteractorData.Transform = InteractorData.RoomSpaceTransform * WorldInteraction->GetRoomTransform();
				break;
			}
		}
	}
}


bool UVREditorInteractor::GetTransformAndForwardVector( FTransform& OutHandTransform, FVector& OutForwardVector ) const
{
	if (bHaveMotionController)
	{
		if (LaserMotionControllerComponent)
		{
			OutHandTransform = InteractorData.Transform;
			OutForwardVector = InteractorData.Transform.TransformVector(FVector::ForwardVector);
		}
		else
		{
			const float LaserPointerRotationOffset = GetHMDDeviceType() == OculusDeviceType
				? VREd::OculusLaserPointerRotationOffset->GetFloat()
				: VREd::ViveLaserPointerRotationOffset->GetFloat();

			OutHandTransform = InteractorData.Transform;
			OutForwardVector = OutHandTransform.GetRotation()
				.RotateVector(FRotator(LaserPointerRotationOffset, 0.0f, 0.0f)
				.RotateVector(FVector::ForwardVector));
		}

		return true;
	}

	return false;
}


/** Changes the color of the buttons on the handmesh */
void UVREditorInteractor::ApplyButtonPressColors( const FViewportActionKeyInput& Action )
{
	const float PressStrength = 10.0f;
	const FName ActionType = Action.ActionType;
	const EInputEvent Event = Action.Event;

	// Trigger
	if (ActionType == ViewportWorldActionTypes::SelectAndMove)
	{
		static FName StaticTriggerParameter( "B1" );
		SetMotionControllerButtonPressedVisuals( Event, StaticTriggerParameter, PressStrength );
	}

	// Shoulder button
	if (ActionType == ViewportWorldActionTypes::WorldMovement)
	{
		static FName StaticShoulderParameter( "B2" );
		SetMotionControllerButtonPressedVisuals( Event, StaticShoulderParameter, PressStrength );
	}

	// Trackpad
	if (ActionType == VRActionTypes::ConfirmRadialSelection)
	{
		static FName StaticTrackpadParameter( "B3" );
		SetMotionControllerButtonPressedVisuals( Event, StaticTrackpadParameter, PressStrength );
	}

	// Modifier
	if (ActionType == VRActionTypes::Modifier)
	{
		static FName StaticModifierParameter( "B4" );
		SetMotionControllerButtonPressedVisuals( Event, StaticModifierParameter, PressStrength );
	}

	if (ActionType == VRActionTypes::Modifier2)
	{
		static FName StaticModifierParameter( "B5" );
		SetMotionControllerButtonPressedVisuals( Event, StaticModifierParameter, PressStrength );
	}
}

void UVREditorInteractor::SetMotionControllerButtonPressedVisuals( const EInputEvent Event, const FName& ParameterName, const float PressStrength )
{
	if (Event == IE_Pressed)
	{
		HandMeshMID->SetScalarParameterValue( ParameterName, PressStrength );
	}
	else if (Event == IE_Released)
	{
		HandMeshMID->SetScalarParameterValue( ParameterName, 0.0f );
	}
}

void UVREditorInteractor::ShowHelpForHand( const bool bShowIt )
{
	if (bShowIt != bWantHelpLabels)
	{
		bWantHelpLabels = bShowIt;

		const FTimespan CurrentTime = FTimespan::FromSeconds( FApp::GetCurrentTime() );
		const FTimespan TimeSinceStartedFadingOut = CurrentTime - HelpLabelShowOrHideStartTime;
		const FTimespan HelpLabelFadeDuration = FTimespan::FromSeconds( VREd::HelpLabelFadeDuration->GetFloat() );

		// If we were already fading, account for that here
		if (TimeSinceStartedFadingOut < HelpLabelFadeDuration)
		{
			// We were already fading, so we'll reverse the time value so it feels continuous
			HelpLabelShowOrHideStartTime = CurrentTime - (HelpLabelFadeDuration - TimeSinceStartedFadingOut);
		}
		else
		{
			HelpLabelShowOrHideStartTime = FTimespan::FromSeconds( FApp::GetCurrentTime() );
		}

		if (bShowIt && HelpLabels.Num() == 0)
		{
			for (const auto& KeyToAction : KeyToActionMap)
			{
				const FKey Key = KeyToAction.Key;
				const FViewportActionKeyInput& Action = KeyToAction.Value;

				UStaticMeshSocket* Socket = FindMeshSocketForKey( HandMeshComponent->GetStaticMesh(), Key );
				if (Socket != nullptr)
				{
					FText LabelText;
					FString ComponentName;

					if (Action.ActionType == VRActionTypes::Modifier)
					{
						LabelText = LOCTEXT( "ModifierHelp", "Modifier" );
						ComponentName = TEXT( "ModifierHelp" );
					}
					else if (Action.ActionType == ViewportWorldActionTypes::WorldMovement)
					{
						LabelText = LOCTEXT( "WorldMovementHelp", "Move World" );
						ComponentName = TEXT( "WorldMovementHelp" );
					}
					else if (Action.ActionType == ViewportWorldActionTypes::SelectAndMove_FullyPressed)
					{
						LabelText = LOCTEXT( "SelectAndMove_FullyPressedHelp", "Select & Move" );
						ComponentName = TEXT( "SelectAndMove_FullyPressedHelp" );
					}
					else if (Action.ActionType == ViewportWorldActionTypes::SelectAndMove)
					{
						LabelText = LOCTEXT( "SelectAndMove_Help", "Select & Move" );
						ComponentName = TEXT( "SelectAndMove_Help" );
					}
					else if (Action.ActionType == VRActionTypes::Touch)
					{
						LabelText = LOCTEXT( "TouchHelp", "Slide" );
						ComponentName = TEXT( "TouchHelp" );
					}
					else if (Action.ActionType == ViewportWorldActionTypes::Undo)
					{
						LabelText = LOCTEXT( "UndoHelp", "Undo" );
						ComponentName = TEXT( "UndoHelp" );
					}
					else if (Action.ActionType == ViewportWorldActionTypes::Redo)
					{
						LabelText = LOCTEXT( "RedoHelp", "Redo" );
						ComponentName = TEXT( "RedoHelp" );
					}
					else if (Action.ActionType == ViewportWorldActionTypes::Delete)
					{
						LabelText = LOCTEXT( "DeleteHelp", "Delete" );
						ComponentName = TEXT( "DeleteHelp" );
					}
					else if (Action.ActionType == VRActionTypes::ConfirmRadialSelection)
					{
						LabelText = LOCTEXT( "ConfirmRadialSelectionHelp", "Radial Menu" );
						ComponentName = TEXT( "ConfirmRadialSelectionHelp" );
					}

					const bool bWithSceneComponent = false;	// Nope, we'll spawn our own inside AFloatingText
					check( VRMode );
					AFloatingText* FloatingText = GetVRMode().SpawnTransientSceneActor<AFloatingText>( ComponentName );
					FloatingText->SetText( LabelText );

					HelpLabels.Add( Key, FloatingText );
				}
			}
		}
	}
}


void UVREditorInteractor::UpdateHelpLabels()
{
	const FTimespan HelpLabelFadeDuration = FTimespan::FromSeconds( VREd::HelpLabelFadeDuration->GetFloat() );

	const FTransform HeadTransform = GetVRMode().GetHeadTransform();

	// Only show help labels if the hand is pretty close to the face
	const float DistanceToHead = (float) (GetTransform().GetLocation() - HeadTransform.GetLocation()).Size();
	const float MinDistanceToHeadForHelp = VREd::HelpLabelFadeDistance->GetFloat() * GetVRMode().GetWorldScaleFactor();	// (in cm)
	bool bShowHelp = VREd::ShowControllerHelpLabels->GetInt() != 0 && DistanceToHead <= MinDistanceToHeadForHelp;

	// Don't show help if a UI is summoned on that hand
	if (HasUIOnForearm() || GetVRMode().GetUISystem().IsShowingRadialMenu( this ))
	{
		bShowHelp = false;
	}

	ShowHelpForHand( bShowHelp );

	// Have the labels finished fading out?  If so, we'll kill their actors!
	const FTimespan CurrentTime = FTimespan::FromSeconds( FApp::GetCurrentTime() );
	const FTimespan TimeSinceStartedFadingOut = CurrentTime - HelpLabelShowOrHideStartTime;
	if (!bWantHelpLabels && (TimeSinceStartedFadingOut > HelpLabelFadeDuration))
	{
		// Get rid of help text
		for (auto& KeyAndValue : HelpLabels)
		{
			AFloatingText* FloatingText = KeyAndValue.Value;
			GetVRMode().DestroyTransientActor( FloatingText );
		}
		HelpLabels.Reset();
	}
	else
	{
		// Update fading state
		float FadeAlpha = FMath::Clamp( (float)TimeSinceStartedFadingOut.GetTotalSeconds() / (float)HelpLabelFadeDuration.GetTotalSeconds(), 0.0f, 1.0f );
		if (!bWantHelpLabels)
		{
			FadeAlpha = 1.0f - FadeAlpha;
		}

		// Exponential falloff, so the fade is really obvious (gamma/HDR)
		FadeAlpha = FMath::Pow( FadeAlpha, 3.0f );

		for (auto& KeyAndValue : HelpLabels)
		{
			const FKey Key = KeyAndValue.Key;
			AFloatingText* FloatingText = KeyAndValue.Value;

			UStaticMeshSocket* Socket = FindMeshSocketForKey( HandMeshComponent->GetStaticMesh(), Key );
			check( Socket != nullptr );
			FTransform SocketRelativeTransform( Socket->RelativeRotation, Socket->RelativeLocation, Socket->RelativeScale );

			// Oculus has asymmetrical controllers, so we the sock transform horizontally
			if (ControllerMotionSource == IMotionController::RightHandSourceId && GetVRMode().GetHMDDeviceType() == OculusDeviceType)
			{
				const FVector Scale3D = SocketRelativeTransform.GetLocation();
				SocketRelativeTransform.SetLocation( FVector( Scale3D.X, -Scale3D.Y, Scale3D.Z ) );
			}

			// Make sure the labels stay the same size even when the world is scaled
			FTransform HandTransformWithWorldToMetersScaling = GetTransform();
			HandTransformWithWorldToMetersScaling.SetScale3D( HandTransformWithWorldToMetersScaling.GetScale3D() * FVector( GetVRMode().GetWorldScaleFactor() ) );

			// Position right on top of the controller itself
			FTransform FloatingTextTransform = SocketRelativeTransform * HandTransformWithWorldToMetersScaling;
			FloatingText->SetActorTransform( FloatingTextTransform );

			// Orientate it toward the viewer
			FloatingText->Update( HeadTransform.GetLocation() );

			// Update fade state
			FloatingText->SetOpacity( FadeAlpha );
		}
	}
}


UStaticMeshSocket* UVREditorInteractor::FindMeshSocketForKey( UStaticMesh* StaticMesh, const FKey Key )
{
	// @todo vreditor: Hard coded mapping of socket names (e.g. "Shoulder") to expected names of sockets in the static mesh
	FName SocketName = NAME_None;
	if (Key == EKeys::Vive_Left_Menu_Click || Key == EKeys::Vive_Right_Menu_Click)
	{
		static FName ShoulderSocketName( "Shoulder" );
		SocketName = ShoulderSocketName;
	}
	else if (Key == VREditorKeyNames::MotionController_Left_FullyPressedTriggerAxis || Key == VREditorKeyNames::MotionController_Right_FullyPressedTriggerAxis ||
		Key == VREditorKeyNames::MotionController_Left_PressedTriggerAxis || Key == VREditorKeyNames::MotionController_Right_PressedTriggerAxis)
	{
		static FName TriggerSocketName( "Trigger" );
		SocketName = TriggerSocketName;
	}
	else if (Key == EKeys::Vive_Left_Grip_Click || Key == EKeys::Vive_Right_Grip_Click ||
		Key == EKeys::ValveIndex_Left_Trackpad_Touch || Key == EKeys::ValveIndex_Right_Trackpad_Touch ||
		Key == EKeys::OculusTouch_Left_Grip_Click || Key == EKeys::OculusTouch_Right_Grip_Click)
	{
		static FName GripSocketName( "Grip" );
		SocketName = GripSocketName;
	}
	else if (Key == EKeys::Vive_Left_Trackpad_Click || Key == EKeys::Vive_Right_Trackpad_Click ||
		Key == EKeys::ValveIndex_Left_Thumbstick_Touch || Key == EKeys::ValveIndex_Right_Thumbstick_Click ||
		Key == EKeys::OculusTouch_Left_Thumbstick_Touch || Key == EKeys::OculusTouch_Right_Thumbstick_Click)
	{
		static FName ThumbstickSocketName( "Thumbstick" );
		SocketName = ThumbstickSocketName;
	}
	else if (Key == EKeys::Vive_Left_Trackpad_Touch || Key == EKeys::Vive_Right_Trackpad_Touch ||
		Key == EKeys::ValveIndex_Left_Thumbstick_Touch || Key == EKeys::ValveIndex_Right_Thumbstick_Touch ||
		Key == EKeys::OculusTouch_Left_Thumbstick_Touch || Key == EKeys::OculusTouch_Right_Thumbstick_Touch)
	{
		static FName TouchSocketName( "Touch" );
		SocketName = TouchSocketName;
	}
	else if (Key == EKeys::Vive_Left_Trackpad_Down || Key == EKeys::Vive_Right_Trackpad_Down ||
		Key == EKeys::ValveIndex_Left_Thumbstick_Down || Key == EKeys::ValveIndex_Right_Thumbstick_Down ||
		Key == EKeys::OculusTouch_Left_Thumbstick_Down || Key == EKeys::OculusTouch_Right_Thumbstick_Down)
	{
		static FName DownSocketName( "Down" );
		SocketName = DownSocketName;
	}
	else if (Key == EKeys::Vive_Left_Trackpad_Up || Key == EKeys::Vive_Right_Trackpad_Up ||
		Key == EKeys::ValveIndex_Left_Thumbstick_Up || Key == EKeys::ValveIndex_Right_Thumbstick_Up ||
		Key == EKeys::OculusTouch_Left_Thumbstick_Up || Key == EKeys::OculusTouch_Right_Thumbstick_Up)
	{
		static FName UpSocketName( "Up" );
		SocketName = UpSocketName;
	}
	else if (Key == EKeys::Vive_Left_Trackpad_Left || Key == EKeys::Vive_Right_Trackpad_Left ||
		Key == EKeys::ValveIndex_Left_Thumbstick_Left || Key == EKeys::ValveIndex_Right_Thumbstick_Left ||
		Key == EKeys::OculusTouch_Left_Thumbstick_Left || Key == EKeys::OculusTouch_Right_Thumbstick_Left)
	{
		static FName LeftSocketName( "Left" );
		SocketName = LeftSocketName;
	}
	else if (Key == EKeys::Vive_Left_Trackpad_Right || Key == EKeys::Vive_Right_Trackpad_Right ||
		Key == EKeys::ValveIndex_Left_Thumbstick_Right || Key == EKeys::ValveIndex_Right_Thumbstick_Right ||
		Key == EKeys::OculusTouch_Left_Thumbstick_Right || Key == EKeys::OculusTouch_Right_Thumbstick_Right)
	{
		static FName RightSocketName( "Right" );
		SocketName = RightSocketName;
	}
	else if (Key == EKeys::ValveIndex_Left_A_Click || Key == EKeys::ValveIndex_Right_A_Click ||
		Key == EKeys::OculusTouch_Left_X_Click || Key == EKeys::OculusTouch_Right_A_Click)
	{
		static FName FaceButton1SocketName( "FaceButton1" );
		SocketName = FaceButton1SocketName;
	}
	else if (Key == EKeys::ValveIndex_Left_B_Click || Key == EKeys::ValveIndex_Right_B_Click ||
		Key == EKeys::OculusTouch_Left_Y_Click || Key == EKeys::OculusTouch_Right_B_Click)
	{
		static FName FaceButton2SocketName( "FaceButton2" );
		SocketName = FaceButton2SocketName;
	}
	else
	{
		// Not a key that we care about
	}

	if (SocketName != NAME_None)
	{
		UStaticMeshSocket* Socket = StaticMesh->FindSocket( SocketName );
		if (Socket != nullptr)
		{
			return Socket;
		}
	}

	return nullptr;
};

void UVREditorInteractor::UpdateSplineLaser( const FVector& InStartLocation, const FVector& InEndLocation, const FVector& InForward )
{
	if (LaserSplineComponent)
	{
		LaserStart = InStartLocation;
		LaserEnd = InEndLocation;

		// Clear the segments before updating it
		LaserSplineComponent->ClearSplinePoints( true );

		const FVector SmoothLaserDirection = InEndLocation - InStartLocation;
		float Distance = (float) SmoothLaserDirection.Size();
		const FVector StraightLaserEndLocation = InStartLocation + (InForward * Distance);
		const int32 NumLaserSplinePoints = LaserSplineMeshComponents.Num();

		LaserSplineComponent->AddSplinePoint( InStartLocation, ESplineCoordinateSpace::Local, false );
		for (int32 Index = 1; Index < NumLaserSplinePoints; Index++)
		{
			float Alpha = (float)Index / (float)NumLaserSplinePoints;
			Alpha = FMath::Sin( Alpha * PI * 0.5f );
			const FVector PointOnStraightLaser = FMath::Lerp( InStartLocation, StraightLaserEndLocation, Alpha );
			const FVector PointOnSmoothLaser = FMath::Lerp( InStartLocation, InEndLocation, Alpha );
			const FVector PointBetweenLasers = FMath::Lerp( PointOnStraightLaser, PointOnSmoothLaser, Alpha );
			LaserSplineComponent->AddSplinePoint( PointBetweenLasers, ESplineCoordinateSpace::Local, false );
		}
		LaserSplineComponent->AddSplinePoint( InEndLocation, ESplineCoordinateSpace::Local, false );

		// Update all the segments of the spline
		LaserSplineComponent->UpdateSpline();

		const float LaserPointerRadius = VREd::LaserPointerRadius->GetFloat() * VRMode->GetWorldScaleFactor();
		Distance *= 0.0001f;
		for (int32 Index = 0; Index < NumLaserSplinePoints; Index++)
		{
			USplineMeshComponent* SplineMeshComponent = LaserSplineMeshComponents[Index];
			check( SplineMeshComponent != nullptr );

			FVector StartLoc, StartTangent, EndLoc, EndTangent;
			LaserSplineComponent->GetLocationAndTangentAtSplinePoint( Index, StartLoc, StartTangent, ESplineCoordinateSpace::Local );
			LaserSplineComponent->GetLocationAndTangentAtSplinePoint( Index + 1, EndLoc, EndTangent, ESplineCoordinateSpace::Local );

			const float AlphaIndex = (float)Index / (float)NumLaserSplinePoints;
			const float AlphaDistance = Distance * AlphaIndex;
			float Radius = LaserPointerRadius * ((AlphaIndex * AlphaDistance) + 1);
			FVector2D LaserScale( Radius, Radius );
			SplineMeshComponent->SetStartScale( LaserScale, false );

			const float NextAlphaIndex = (float)(Index + 1) / (float)NumLaserSplinePoints;
			const float NextAlphaDistance = Distance * NextAlphaIndex;
			Radius = LaserPointerRadius * ((NextAlphaIndex * NextAlphaDistance) + 1);
			LaserScale = FVector2D( Radius, Radius );
			SplineMeshComponent->SetEndScale( LaserScale, false );

			SplineMeshComponent->SetStartAndEnd( StartLoc, StartTangent, EndLoc, EndTangent, true );
		}
	}
}

void UVREditorInteractor::SetLaserVisibility( const bool bVisible )
{
	for (USplineMeshComponent* SplineMeshComponent : LaserSplineMeshComponents)
	{
		SplineMeshComponent->SetVisibility( bVisible );
	}
}

void UVREditorInteractor::SetLaserVisuals( const FLinearColor& NewColor, const float CrawlFade, const float CrawlSpeed )
{
	static FName StaticLaserColorParameterName( "LaserColor" );
	LaserPointerMID->SetVectorParameterValue( StaticLaserColorParameterName, NewColor );
	TranslucentLaserPointerMID->SetVectorParameterValue( StaticLaserColorParameterName, NewColor );

	static FName StaticCrawlParameterName( "Crawl" );
	LaserPointerMID->SetScalarParameterValue( StaticCrawlParameterName, CrawlFade );
	TranslucentLaserPointerMID->SetScalarParameterValue( StaticCrawlParameterName, CrawlFade );

	static FName StaticCrawlSpeedParameterName( "CrawlSpeed" );
	LaserPointerMID->SetScalarParameterValue( StaticCrawlSpeedParameterName, CrawlSpeed );
	TranslucentLaserPointerMID->SetScalarParameterValue( StaticCrawlSpeedParameterName, CrawlSpeed );

	static FName StaticHandTrimColorParameter( "TrimGlowColor" );
	HandMeshMID->SetVectorParameterValue( StaticHandTrimColorParameter, NewColor );

	HoverPointLightComponent->SetLightColor( NewColor );
}

void UVREditorInteractor::UpdateRadialMenuInput( const float DeltaTime )
{
	UVREditorUISystem& UISystem = GetVRMode().GetUISystem();
	const FName HMDDeviceType = GetVRMode().GetHMDDeviceType();
	//Update the radial menu
	EViewportInteractionDraggingMode DraggingMode = GetDraggingMode();
	if (GetControllerType() == EControllerType::UI)
	{
		if ((bIsTrackpadPositionValid[0] && bIsTrackpadPositionValid[1]) &&
			DraggingMode != EViewportInteractionDraggingMode::AssistingDrag)
		{
			if (bIsScrubbingSequence)
			{
				const FVector2D ReturnToCenter = FVector2D::ZeroVector;
				UISystem.GetRadialMenuFloatingUI()->HighlightSlot( ReturnToCenter );

				const float NewPlayRate = (float) FMath::GetMappedRangeValueClamped( FVector2D( -1.0f, 1.0f ), FVector2D( -1.0f*VREd::SequencerScrubMax->GetFloat(), VREd::SequencerScrubMax->GetFloat() ), TrackpadPosition.X );
				FVREditorActionCallbacks::PlaySequenceAtRate( VRMode, NewPlayRate );
			}
			else
			{
				// Update the radial menu if we are already showing the radial menu
				if (UISystem.IsShowingRadialMenu( this ))
				{
					if (UISystem.GetRadialMenuFloatingUI()->GetWidgetComponents().Num() > 0)
					{
						UISystem.GetRadialMenuFloatingUI()->HighlightSlot( TrackpadPosition );

						if (TrackpadPosition.GetAbsMax() > VREd::MinJoystickOffsetBeforeFlick->GetFloat())
						{
							LastActiveTrackpadUpdateTime = FTimespan::FromSeconds( FPlatformTime::Seconds() );
						}
					}
				}
				else if (!UISystem.IsShowingRadialMenu( this ) &&
					TrackpadPosition.GetAbsMax() > VREd::MinJoystickOffsetBeforeFlick->GetFloat())
				{
					const bool bForceRefresh = false;
					UISystem.TryToSpawnRadialMenu( this, bForceRefresh );
					LastActiveTrackpadUpdateTime = FTimespan::FromSeconds( FPlatformTime::Seconds() );
				}
			}
		}
		// If we are not currently touching the Vive touchpad, reset the highlighted button and pause sequencer playback if scrubbing
		else if (HMDDeviceType == SteamVRDeviceType && !IsActionKeyPressed(VRActionTypes::Touch))
		{
			if (bIsScrubbingSequence)
			{
				FVREditorActionCallbacks::PauseSequencePlayback( VRMode );
			}

			if (UISystem.IsShowingRadialMenu( this ))
			{
				const FVector2D ReturnToCenter = FVector2D::ZeroVector;
				UISystem.GetRadialMenuFloatingUI()->HighlightSlot( ReturnToCenter );
			}
		}
	}

	else if (GetControllerType() == EControllerType::Laser)
	{
		if (HMDDeviceType != SteamVRDeviceType &&
			(bIsTrackpadPositionValid[0] && bIsTrackpadPositionValid[1]) &&
			DraggingMode != EViewportInteractionDraggingMode::TransformablesWithGizmo &&
			DraggingMode != EViewportInteractionDraggingMode::TransformablesFreely &&
			DraggingMode != EViewportInteractionDraggingMode::TransformablesAtLaserImpact &&
			DraggingMode != EViewportInteractionDraggingMode::AssistingDrag &&
			!VRMode->IsAimingTeleport())
		{
			// Move thumbstick left to undo
			if (TrackpadPosition.X < -1 * VREd::MinJoystickOffsetBeforeFlick->GetFloat() &&
				bFlickActionExecuted == false &&
				!IsHoveringOverUI())
			{
				if (bIsUndoRedoSwipeEnabled)
				{
					VRMode->GetWorldInteraction().Undo();
				}
				bFlickActionExecuted = true;
			}
			// Move thumbstick right to redo
			if (TrackpadPosition.X > VREd::MinJoystickOffsetBeforeFlick->GetFloat() &&
				bFlickActionExecuted == false &&
				!IsHoveringOverUI())
			{
				if (bIsUndoRedoSwipeEnabled)
				{
					VRMode->GetWorldInteraction().Redo();
				}
				bFlickActionExecuted = true;
			}
			// Center to reset
			// TODO: Remove finger from touchpad to reset vive
			if (FMath::IsNearlyZero( TrackpadPosition.X ) &&
				!IsHoveringOverUI())
			{
				bFlickActionExecuted = false;
			}
		}
	}
}

void UVREditorInteractor::UndoRedoFromSwipe( const ETouchSwipeDirection InSwipeDirection )
{
	EViewportInteractionDraggingMode DraggingMode = GetDraggingMode();
	if (GetControllerType() == EControllerType::Laser &&
		VRMode->GetHMDDeviceType() == SteamVRDeviceType &&
		DraggingMode != EViewportInteractionDraggingMode::TransformablesWithGizmo &&
		DraggingMode != EViewportInteractionDraggingMode::TransformablesFreely &&
		DraggingMode != EViewportInteractionDraggingMode::TransformablesAtLaserImpact &&
		DraggingMode != EViewportInteractionDraggingMode::AssistingDrag &&
		!VRMode->IsAimingTeleport())
	{
		if (InSwipeDirection == ETouchSwipeDirection::Left)
		{
			VRMode->GetWorldInteraction().Undo();
			bFlickActionExecuted = true;
		}
		else if (InSwipeDirection == ETouchSwipeDirection::Right)
		{
			VRMode->GetWorldInteraction().Redo();
			bFlickActionExecuted = true;
		}
	}
}

bool UVREditorInteractor::GetIsLaserBlocked() const
{
	return Super::GetIsLaserBlocked() || (GetControllerType() != EControllerType::Laser && GetControllerType() != EControllerType::AssistingLaser);
}


void UVREditorInteractor::ResetTrackpad()
{
	TrackpadPosition = FVector2D::ZeroVector;
	bIsTrackpadPositionValid[0] = false;
	bIsTrackpadPositionValid[1] = false;
}

bool UVREditorInteractor::IsTouchingTrackpad() const
{
	return IsActionKeyPressed(VRActionTypes::Touch);
}

FVector2D UVREditorInteractor::GetTrackpadPosition() const
{
	return TrackpadPosition;
}

FVector2D UVREditorInteractor::GetLastTrackpadPosition() const
{
	return LastTrackpadPosition;
}

bool UVREditorInteractor::IsTrackpadPositionValid( const int32 AxisIndex ) const
{
	return bIsTrackpadPositionValid[AxisIndex];
}

FTimespan& UVREditorInteractor::GetLastTrackpadPositionUpdateTime()
{
	return LastTrackpadPositionUpdateTime;
}

FTimespan& UVREditorInteractor::GetLastActiveTrackpadUpdateTime()
{
	return LastActiveTrackpadUpdateTime;
}

const FVector& UVREditorInteractor::GetLaserStart() const
{
	return LaserStart;
}

const FVector& UVREditorInteractor::GetLaserEnd() const
{
	return LaserEnd;
}

void UVREditorInteractor::SetForceLaserColor( const FLinearColor& InColor )
{
	ForceLaserColor = InColor;
}

AVREditorTeleporter* UVREditorInteractor::GetTeleportActor()
{
	return VRMode->GetTeleportActor();
}

bool UVREditorInteractor::IsHoveringOverUI() const
{
	return bIsHoveringOverUI;
}

void UVREditorInteractor::SetHasUIOnForearm( const bool bInHasUIOnForearm )
{
	bHasUIOnForearm = bInHasUIOnForearm;
}

bool UVREditorInteractor::HasUIOnForearm() const
{
	return bHasUIOnForearm;
}

UWidgetComponent* UVREditorInteractor::GetLastHoveredWidgetComponent() const
{
	return InteractorData.LastHoveredWidgetComponent.Get();
}

void UVREditorInteractor::SetLastHoveredWidgetComponent( UWidgetComponent* NewHoveringOverWidgetComponent )
{
	InteractorData.LastHoveredWidgetComponent = NewHoveringOverWidgetComponent;
}

bool UVREditorInteractor::IsModifierPressed() const
{
	return IsActionKeyPressed(VRActionTypes::Modifier);
}

void UVREditorInteractor::SetIsClickingOnUI( const bool bInIsClickingOnUI )
{
	bIsClickingOnUI = bInIsClickingOnUI;
}

bool UVREditorInteractor::IsClickingOnUI() const
{
	return bIsClickingOnUI;
}

void UVREditorInteractor::SetIsHoveringOverUI( const bool bInIsHoveringOverUI )
{
	bIsHoveringOverUI = bInIsHoveringOverUI;
}

void UVREditorInteractor::SetIsRightClickingOnUI( const bool bInIsRightClickingOnUI )
{
	bIsRightClickingOnUI = bInIsRightClickingOnUI;
}

bool UVREditorInteractor::IsRightClickingOnUI() const
{
	return bIsRightClickingOnUI;
}

void UVREditorInteractor::SetLastUIPressTime( const double InLastUIPressTime )
{
	LastUIPressTime = InLastUIPressTime;
}

double UVREditorInteractor::GetLastUIPressTime() const
{
	return LastUIPressTime;
}

void UVREditorInteractor::SetUIScrollVelocity( const float InUIScrollVelocity )
{
	UIScrollVelocity = InUIScrollVelocity;
}

float UVREditorInteractor::GetUIScrollVelocity() const
{
	return UIScrollVelocity;
}

float UVREditorInteractor::GetSelectAndMoveTriggerValue() const
{
	return SelectAndMoveTriggerValue;
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
