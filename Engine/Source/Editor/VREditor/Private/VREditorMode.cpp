// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorMode.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Docking/SDockTab.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Components/SpotLightComponent.h"
#include "GameFramework/WorldSettings.h"
#include "DrawDebugHelpers.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "UI/VREditorUISystem.h"
#include "VIBaseTransformGizmo.h"
#include "ViewportWorldInteraction.h"
#include "VREditorPlacement.h"
#include "VREditorAvatarActor.h"
#include "Teleporter/VREditorTeleporter.h"
#include "Teleporter/VREditorAutoScaler.h"
#include "VREditorStyle.h"
#include "VREditorAssetContainer.h"
#include "Framework/Notifications/NotificationManager.h"
#include "CameraController.h"
#include "EngineGlobals.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "SLevelViewport.h"
#include "MotionControllerComponent.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "Interfaces/IProjectManager.h"

#include "IViewportInteractionModule.h"
#include "VREditorInteractor.h"

#include "EditorWorldExtension.h"
#include "SequencerSettings.h"
#include "Kismet/GameplayStatics.h"
#include "Components/AudioComponent.h"
#include "Components/StaticMeshComponent.h"
#include "VREditorActions.h"
#include "EditorModes.h"
#include "VRModeSettings.h"
#include "IVREditorModule.h"
#include "Engine/StaticMeshActor.h"
#include "EngineUtils.h"
#include "Kismet/GameplayStatics.h"
#include "IMotionController.h"
#include "UI/VREditorFloatingUI.h"
#include "AssetEditorViewportLayout.h"
#include "LevelViewportActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorMode)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "VREditorMode"

namespace VREd
{
	FAutoConsoleVariable DefaultVRNearClipPlane(TEXT("VREd.DefaultVRNearClipPlane"), 5.0f, TEXT("The near clip plane to use for VR"));
	static FAutoConsoleVariable SlateDragDistanceOverride( TEXT( "VREd.SlateDragDistanceOverride" ), 40.0f, TEXT( "How many pixels you need to drag before a drag and drop operation starts in VR" ) );
	FAutoConsoleVariable DefaultWorldToMeters(TEXT("VREd.DefaultWorldToMeters"), 100.0f, TEXT("Default world to meters scale"));

	static FAutoConsoleVariable ShowHeadVelocity( TEXT( "VREd.ShowHeadVelocity" ), 0, TEXT( "Whether to draw a debug indicator that shows how much the head is accelerating" ) );
	static FAutoConsoleVariable HeadVelocitySmoothing( TEXT( "VREd.HeadVelocitySmoothing" ), 0.95f, TEXT( "How much to smooth out head velocity data" ) );
	static FAutoConsoleVariable HeadVelocityMinRadius( TEXT( "VREd.HeadVelocityMinRadius" ), 0.0f, TEXT( "How big the inner circle of the head velocity ring should be" ) );
	static FAutoConsoleVariable HeadVelocityMaxRadius( TEXT( "VREd.HeadVelocityMaxRadius" ), 10.0f, TEXT( "How big the outer circle of the head velocity ring should be" ) );
	static FAutoConsoleVariable HeadVelocityMinLineThickness( TEXT( "VREd.HeadVelocityMinLineThickness" ), 0.05f, TEXT( "How thick the head velocity ring lines should be" ) );
	static FAutoConsoleVariable HeadVelocityMaxLineThickness( TEXT( "VREd.HeadVelocityMaxLineThickness" ), 0.4f, TEXT( "How thick the head velocity ring lines should be" ) );
	static FAutoConsoleVariable HeadLocationMaxVelocity( TEXT( "VREd.HeadLocationMaxVelocity" ), 25.0f, TEXT( "For head velocity indicator, the maximum location velocity in cm/s" ) );
	static FAutoConsoleVariable HeadRotationMaxVelocity( TEXT( "VREd.HeadRotationMaxVelocity" ), 80.0f, TEXT( "For head velocity indicator, the maximum rotation velocity in degrees/s" ) );
	static FAutoConsoleVariable HeadLocationVelocityOffset( TEXT( "VREd.HeadLocationVelocityOffset" ), TEXT( "X=20, Y=0, Z=5" ), TEXT( "Offset relative to head for location velocity debug indicator" ) );
	static FAutoConsoleVariable HeadRotationVelocityOffset( TEXT( "VREd.HeadRotationVelocityOffset" ), TEXT( "X=20, Y=0, Z=-5" ), TEXT( "Offset relative to head for rotation velocity debug indicator" ) );
	static FAutoConsoleVariable SFXMultiplier(TEXT("VREd.SFXMultiplier"), 1.5f, TEXT("Default Sound Effect Volume Multiplier"));

	static FAutoConsoleCommand ToggleDebugMode(TEXT("VREd.ToggleDebugMode"), TEXT("Toggles debug mode of the VR Mode"), FConsoleCommandDelegate::CreateStatic(&UVREditorMode::ToggleDebugMode));
}

const TCHAR* UVREditorMode::AssetContainerPath = TEXT("/Engine/VREditor/VREditorAssetContainerData");
bool UVREditorMode::bDebugModeEnabled = false;

UVREditorMode::UVREditorMode() :
	Super(),
	bWantsToExitMode( false ),
	bIsFullyInitialized( false ),
	AppTimeModeEntered( FTimespan::Zero() ),
	AvatarActor( nullptr ),
	FlashlightComponent( nullptr ),
	bIsFlashlightOn( false ),
	MotionControllerID( 0 ),	// @todo vreditor minor: We only support a single controller, and we assume the first controller are the motion controls
	UISystem( nullptr ),
	TeleportActor( nullptr ),
	AutoScalerSystem( nullptr ),
	WorldInteraction( nullptr ),
	InteractorClass( UVREditorInteractor::StaticClass() ),
	TeleporterClass( AVREditorTeleporter::StaticClass() ),
	bFirstTick( true ),
	AssetContainer( nullptr )
{
	SetActive(false);
}

void UVREditorMode::SetHMDDeviceTypeOverride( FName InOverrideType )
{
	ensureMsgf(!IsActive(), TEXT("HMD device type override should be specified before VR editor mode is entered."));
	HMDDeviceTypeOverride = InOverrideType;
}

void UVREditorMode::Init()
{
	Super::Init();

	bIsFullyInitialized = false;
	bWantsToExitMode = false;

	AppTimeModeEntered = FTimespan::FromSeconds( FApp::GetCurrentTime() );

	// Setting up colors
	Colors.SetNumZeroed( (int32)EColors::TotalCount );
	{	
		Colors[ (int32)EColors::DefaultColor ] = FLinearColor(0.701f, 0.084f, 0.075f, 1.0f);	
		Colors[ (int32)EColors::SelectionColor ] = FLinearColor(1.0f, 0.467f, 0.0f, 1.f);
		Colors[ (int32)EColors::WorldDraggingColor ] = FLinearColor(0.106f, 0.487f, 0.106f, 1.0f);
		Colors[ (int32)EColors::UIColor ] = FLinearColor(0.22f, 0.7f, 0.98f, 1.0f);
		Colors[ (int32)EColors::UISelectionBarColor ] = FLinearColor( 0.025f, 0.025f, 0.025f, 1.0f );
		Colors[ (int32)EColors::UISelectionBarHoverColor ] = FLinearColor( 0.1f, 0.1f, 0.1f, 1.0f );
		Colors[ (int32)EColors::UICloseButtonColor ] = FLinearColor( 0.1f, 0.1f, 0.1f, 1.0f );
		Colors[ (int32)EColors::UICloseButtonHoverColor ] = FLinearColor( 1.0f, 1.0f, 1.0f, 1.0f );
	}

	{
		UEditorWorldExtensionCollection* Collection = GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(GetWorld());
		check(Collection != nullptr);

		// Add viewport world interaction to the collection if not already there
		WorldInteraction = Cast<UViewportWorldInteraction>(Collection->FindExtension(UViewportWorldInteraction::StaticClass()));
		if (WorldInteraction == nullptr)
		{
			WorldInteraction = NewObject<UViewportWorldInteraction>(Collection);
			check(WorldInteraction != nullptr);

			Collection->AddExtension(WorldInteraction);
			bAddedViewportWorldInteractionExtension = true;
		}
		else
		{
			WorldInteraction->UseVWInteractions();
		}

		check( WorldInteraction != nullptr );
	}

	// Setup the asset container.
	AssetContainer = &LoadAssetContainer();
	check(AssetContainer != nullptr);

	// Setup slate style
	FVREditorStyle::Get();

	bIsFullyInitialized = true;
}

/*
* @EventName Editor.Usage.EnterVRMode
*
* @Trigger Entering VR editing mode
*
* @Type Client
*
* @EventParam HMDDevice (string) The name of the HMD Device type
*
* @Source Editor
*
* @Owner Lauren.Ridge
*
*/
void UVREditorMode::Shutdown()
{
	bIsFullyInitialized = false;
	
	AvatarActor = nullptr;
	FlashlightComponent = nullptr;
	UISystem = nullptr;
	TeleportActor = nullptr;
	AutoScalerSystem = nullptr;
	WorldInteraction = nullptr;
	AssetContainer = nullptr;

	Super::Shutdown();
}

void UVREditorMode::AllocateInteractors()
{
	class UVREditorInteractor* LeftHandInteractor = nullptr;
	class UVREditorInteractor* RightHandInteractor = nullptr;

	InteractorClass.LoadSynchronous();

	if (InteractorClass.IsValid())
	{
		LeftHandInteractor = NewObject<UVREditorInteractor>(GetTransientPackage(), InteractorClass.Get());
		RightHandInteractor = NewObject<UVREditorInteractor>(GetTransientPackage(), InteractorClass.Get());
	}

	if (LeftHandInteractor == nullptr)
	{
		LeftHandInteractor = NewObject<UVREditorInteractor>();
	}

	if (RightHandInteractor == nullptr)
	{
		RightHandInteractor = NewObject<UVREditorInteractor>();
	}

	check( LeftHandInteractor );
	check( RightHandInteractor );

	Interactors.Add( LeftHandInteractor );
	Interactors.Add( RightHandInteractor );

	LeftHandInteractor->SetControllerHandSide(IMotionController::LeftHandSourceId );
	RightHandInteractor->SetControllerHandSide( IMotionController::RightHandSourceId );

	for (UVREditorInteractor* Interactor : Interactors)
	{
		Interactor->Init( this );
		WorldInteraction->AddInteractor( Interactor );
	}

	WorldInteraction->PairInteractors( LeftHandInteractor, RightHandInteractor );
}

void UVREditorMode::Enter()
{
	BeginEntry();
	SetupSubsystems();
	FinishEntry();
}

namespace UE::VREditor::Private
{
	// Defined in VREditorModeBase.cpp.
	TSharedPtr<SLevelViewport> TryGetActiveViewport();
}

void UVREditorMode::BeginEntry()
{
	using namespace UE::VREditor::Private;

	FSavedEditorState& SavedEditorState = static_cast<FSavedEditorState&>(SavedEditorStateChecked());

	bWantsToExitMode = false;

	{
		WorldInteraction->OnPreWorldInteractionTick().AddUObject(this, &UVREditorMode::PreTick);
		WorldInteraction->OnPostWorldInteractionTick().AddUObject(this, &UVREditorMode::PostTick);
	}


	// @todo vreditor: We need to make sure the user can never switch to orthographic mode, or activate settings that
	// would disrupt the user's ability to view the VR scene.

	// @todo vreditor: Don't bother drawing toolbars in VR, or other things that won't matter in VR

	{
		const TSharedRef< ILevelEditor >& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstLevelEditor().ToSharedRef();

		// Do we have an active perspective viewport that is valid for VR?  If so, go ahead and use that.
		TSharedPtr<SLevelViewport> ExistingActiveLevelViewport = TryGetActiveViewport();

		if (ExistingActiveLevelViewport && ExistingActiveLevelViewport->GetCommandList() && FLevelViewportCommands::Get().SetDefaultViewportType)
		{
			ExistingActiveLevelViewport->GetCommandList()->TryExecuteAction(
				FLevelViewportCommands::Get().SetDefaultViewportType.ToSharedRef());

			// If the active viewport was e.g. a cinematic viewport, changing it
			// back to default recreated it and our pointer might be stale.
			ExistingActiveLevelViewport = TryGetActiveViewport();
		}

		if (!ensure(ExistingActiveLevelViewport))
		{
			return;
		}

		ExistingActiveLevelViewport->RemoveAllPreviews(true);

		StartViewport(ExistingActiveLevelViewport);

		if (WorldInteraction != nullptr)
		{
			WorldInteraction->SetDefaultOptionalViewportClient(ExistingActiveLevelViewport->GetViewportClient());
		}

		if (bActuallyUsingVR)
		{
			// Tell Slate to require a larger pixel distance threshold before the drag starts.  This is important for things
			// like Content Browser drag and drop.
			SavedEditorState.DragTriggerDistance = FSlateApplication::Get().GetDragTriggerDistance();
			FSlateApplication::Get().SetDragTriggerDistance(VREd::SlateDragDistanceOverride->GetFloat());

			// When actually in VR, make sure the transform gizmo is big!
			SavedEditorState.TransformGizmoScale = WorldInteraction->GetTransformGizmoScale();
			WorldInteraction->SetTransformGizmoScale(GetDefault<UVRModeSettings>()->GizmoScale);
			WorldInteraction->SetShouldSuppressExistingCursor(true);
			WorldInteraction->SetInVR(true);
		}
	}

	// Switch us back to default mode and close any open sequencer windows
	FVREditorActionCallbacks::ChangeEditorModes(FBuiltinEditorModes::EM_Default);
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	if (LevelEditorModule.GetLevelEditorTabManager())
	{
		if (TSharedPtr<SDockTab> SequencerTab = LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(FTabId("Sequencer")))
		{
			SequencerTab->RequestCloseTab();
		}
	}

	// Setup our avatar
	if (AvatarActor == nullptr)
	{
		const bool bWithSceneComponent = true;
		AvatarActor = SpawnTransientSceneActor<AVREditorAvatarActor>(TEXT("AvatarActor"), bWithSceneComponent);
		AvatarActor->Init(this);

		WorldInteraction->AddActorToExcludeFromHitTests(AvatarActor);
	}

	// If we're actually using VR, go ahead and disable notifications.  We won't be able to see them in VR
	// currently, and they can introduce performance issues if they pop up on the desktop
	if (bActuallyUsingVR)
	{
		FSlateNotificationManager::Get().SetAllowNotifications(false);
	}

	/** This will make sure this is not ticking after the editor has been closed. */
	GEditor->OnEditorClose().AddUObject(this, &UVREditorMode::OnEditorClosed);
}

void UVREditorMode::SetupSubsystems()
{
	// Setup world interaction
	// We need input preprocessing for VR so that we can receive motion controller input without any viewports having 
	// to be focused.  This is mainly because Slate UI injected into the 3D world can cause focus to be lost unexpectedly,
	// but we need the user to still be able to interact with UI.
	WorldInteraction->SetUseInputPreprocessor( true );

	// Motion controllers
	AllocateInteractors();

	if( bActuallyUsingVR )
	{
		// When actually using VR devices, we don't want a mouse cursor interactor
		WorldInteraction->ReleaseMouseCursorInteractor();
	}

	// Setup the UI system
	UISystem = NewObject<UVREditorUISystem>();
	UISystem->Init(this);

	PlacementSystem = NewObject<UVREditorPlacement>();
	PlacementSystem->Init(this);

	// Setup teleporter
	TeleporterClass.LoadSynchronous();

	if (TeleporterClass.IsValid())
	{
		TeleportActor = CastChecked<AVREditorTeleporter>( SpawnTransientSceneActor(TeleporterClass.Get(), TEXT( "Teleporter" ), true ) );
	}

	if( !TeleportActor )
	{
		TeleportActor = SpawnTransientSceneActor<AVREditorTeleporter>( TEXT( "Teleporter" ), true );
	}

	check( TeleportActor );
	TeleportActor->Init( this );
	WorldInteraction->AddActorToExcludeFromHitTests( TeleportActor );

	// Setup autoscaler
	AutoScalerSystem = NewObject<UVREditorAutoScaler>();
	AutoScalerSystem->Init( this );

	for (UVREditorInteractor* Interactor : Interactors)
	{
		Interactor->SetupComponent( AvatarActor );
	}
}

void UVREditorMode::FinishEntry()
{
	bFirstTick = true;
	SetActive(true);
	OnVRModeEntryCompleteEvent.Broadcast();
}

void UVREditorMode::Exit(const bool bShouldDisableStereo)
{
	const FSavedEditorState& SavedEditorState = static_cast<const FSavedEditorState&>(SavedEditorStateChecked());

	{
		if (TSharedPtr<SLevelViewport> VrViewport = GetVrLevelViewport())
		{
			VrViewport->RemoveAllPreviews(false);
		}

		GEditor->SelectNone(true, true, false);
		GEditor->NoteSelectionChange();
		FVREditorActionCallbacks::ChangeEditorModes(FBuiltinEditorModes::EM_Default);
		
		//Destroy the avatar
		{
			DestroyTransientActor(AvatarActor);
			AvatarActor = nullptr;
			FlashlightComponent = nullptr;
		}

		{
			if(bActuallyUsingVR)
			{
				// Restore Slate drag trigger distance
				FSlateApplication::Get().SetDragTriggerDistance( SavedEditorState.DragTriggerDistance );

				// Restore gizmo size
				WorldInteraction->SetTransformGizmoScale( SavedEditorState.TransformGizmoScale );
				WorldInteraction->SetShouldSuppressExistingCursor(false);
			}

			CloseViewport( bShouldDisableStereo );

			VREditorLevelViewportWeakPtr.Reset();
			OnVREditingModeExit_Handler.ExecuteIfBound();
		}

		// Kill the VR editor window
		TSharedPtr<SWindow> VREditorWindow( VREditorWindowWeakPtr.Pin() );
		if(VREditorWindow.IsValid())
		{
			VREditorWindow->RequestDestroyWindow();
			VREditorWindow.Reset();
		}
	}

	// Kill subsystems
	if( UISystem != nullptr )
	{
		UISystem->Shutdown();
		UISystem->MarkAsGarbage();
		UISystem = nullptr;
	}

	if( PlacementSystem != nullptr )
	{
		PlacementSystem->Shutdown();
		PlacementSystem->MarkAsGarbage();
		PlacementSystem = nullptr;
	}

	if( TeleportActor != nullptr )
	{
		DestroyTransientActor( TeleportActor );
		TeleportActor = nullptr;
	}

	if( AutoScalerSystem != nullptr )
	{
		AutoScalerSystem->Shutdown();
		AutoScalerSystem->MarkAsGarbage();
		AutoScalerSystem = nullptr;
	}

	if( WorldInteraction != nullptr )
	{
		WorldInteraction->SetUseInputPreprocessor( false );

		WorldInteraction->OnHandleKeyInput().RemoveAll( this );
		WorldInteraction->OnPreWorldInteractionTick().RemoveAll( this );
		WorldInteraction->OnPostWorldInteractionTick().RemoveAll( this );

		for (UVREditorInteractor* Interactor : Interactors)
		{
			WorldInteraction->RemoveInteractor( Interactor );
			Interactor->MarkAsGarbage();
		}
		Interactors.Empty();
		
		// Restore the mouse cursor if we removed it earlier
		if( bActuallyUsingVR )
		{
			WorldInteraction->AddMouseCursorInteractor();
			WorldInteraction->SetInVR(false);
		}

		UEditorWorldExtensionCollection* Collection = GetOwningCollection();
		check(Collection != nullptr);

		if (bAddedViewportWorldInteractionExtension)
		{
			Collection->RemoveExtension(WorldInteraction);
			bAddedViewportWorldInteractionExtension = false;
		}
		else
		{
			WorldInteraction->UseLegacyInteractions();
		}

		WorldInteraction = nullptr;
	}

	if( bActuallyUsingVR )
	{
		FSlateNotificationManager::Get().SetAllowNotifications( true);
	}

	AssetContainer = nullptr;


	GEditor->OnEditorClose().RemoveAll( this );

	const bool bIsInPIEOrSimulate = (GEditor->PlayWorld != nullptr) || (GEditor->bIsSimulatingInEditor);
	if (bIsInPIEOrSimulate)
	{
		GEditor->RequestEndPlayMap();
	}

	bWantsToExitMode = false;
	SetActive(false);
	bFirstTick = false;
}

void UVREditorMode::OnEditorClosed()
{
	if(IsActive())
	{
		Exit( false );
		Shutdown();
	}
}

void UVREditorMode::StartExitingVRMode()
{
	bWantsToExitMode = true;
}

void UVREditorMode::OnVREditorWindowClosed( const TSharedRef<SWindow>& ClosedWindow )
{
	StartExitingVRMode();
}

void UVREditorMode::PreTick( const float DeltaTime )
{
	if( !bIsFullyInitialized || !IsActive() || bWantsToExitMode )
	{
		return;
	}

	const FSavedEditorState& SavedEditorState = static_cast<const FSavedEditorState&>(SavedEditorStateChecked());

	//Setting the initial position and rotation based on the editor viewport when going into VR mode
	if( bFirstTick && bActuallyUsingVR )
	{
		const FTransform RoomToWorld = GetRoomTransform();
		const FTransform WorldToRoom = RoomToWorld.Inverse();
		FTransform ViewportToWorld = FTransform( SavedEditorState.ViewRotation, SavedEditorState.ViewLocation );
		FTransform ViewportToRoom = ( ViewportToWorld * WorldToRoom );

		FTransform ViewportToRoomYaw = ViewportToRoom;
		ViewportToRoomYaw.SetRotation( FQuat( FRotator( 0.0f, ViewportToRoomYaw.GetRotation().Rotator().Yaw, 0.0f ) ) );

		FTransform HeadToRoomYaw = GetRoomSpaceHeadTransform();
		HeadToRoomYaw.SetRotation( FQuat( FRotator( 0.0f, HeadToRoomYaw.GetRotation().Rotator().Yaw, 0.0f ) ) );

		FTransform RoomToWorldYaw = RoomToWorld;
		RoomToWorldYaw.SetRotation( FQuat( FRotator( 0.0f, RoomToWorldYaw.GetRotation().Rotator().Yaw, 0.0f ) ) );

		FTransform ResultToWorld = ( HeadToRoomYaw.Inverse() * ViewportToRoomYaw ) * RoomToWorldYaw;
		SetRoomTransform( ResultToWorld );
	}
}

void UVREditorMode::PostTick( float DeltaTime )
{
	if( !bIsFullyInitialized || !IsActive() || bWantsToExitMode || !VREditorLevelViewportWeakPtr.IsValid() )
	{
		return;
	}

	TickHandle.Broadcast( DeltaTime );
	UISystem->Tick( GetLevelViewportPossessedForVR().GetViewportClient().Get(), DeltaTime );

	// Update avatar meshes
	{
		// Move our avatar mesh along with the room.  We need our hand components to remain the same coordinate space as the 
		AvatarActor->SetActorTransform( GetRoomTransform() );
		AvatarActor->TickManually( DeltaTime );


	}

	// Updating the scale and intensity of the flashlight according to the world scale
	if (FlashlightComponent)
	{
		float CurrentFalloffExponent = FlashlightComponent->LightFalloffExponent;
		//@todo vreditor tweak
		float UpdatedFalloffExponent = FMath::Clamp(CurrentFalloffExponent / GetWorldScaleFactor(), 2.0f, 16.0f);
		FlashlightComponent->SetLightFalloffExponent(UpdatedFalloffExponent);
	}

	if( WorldInteraction->HaveHeadTransform() && VREd::ShowHeadVelocity->GetInt() != 0 )
	{
		const FTransform RoomSpaceHeadToWorld = WorldInteraction->GetRoomSpaceHeadTransform();
		static FTransform LastRoomSpaceHeadToWorld = RoomSpaceHeadToWorld;

		const float WorldScaleFactor = WorldInteraction->GetWorldScaleFactor();
		static float LastWorldScaleFactor = WorldScaleFactor;

		const float MinInnerRadius = VREd::HeadVelocityMinRadius->GetFloat() * WorldScaleFactor;
		const float MaxOuterRadius = VREd::HeadVelocityMaxRadius->GetFloat() * WorldScaleFactor;
		const float MinThickness = VREd::HeadVelocityMinLineThickness->GetFloat() * WorldScaleFactor;
		const float MaxThickness = VREd::HeadVelocityMaxLineThickness->GetFloat() * WorldScaleFactor;

		const float MaxLocationVelocity = VREd::HeadLocationMaxVelocity->GetFloat();	// cm/s
		const float MaxRotationVelocity = VREd::HeadRotationMaxVelocity->GetFloat();	// degrees/s

		const float LocationVelocity = (float) ( LastRoomSpaceHeadToWorld.GetLocation() / LastWorldScaleFactor - RoomSpaceHeadToWorld.GetLocation() / WorldScaleFactor ).Size() / DeltaTime;

		const float YawVelocity = (float) FMath::Abs( FMath::FindDeltaAngleDegrees( LastRoomSpaceHeadToWorld.GetRotation().Rotator().Yaw, RoomSpaceHeadToWorld.GetRotation().Rotator().Yaw ) ) / DeltaTime;
		const float PitchVelocity = (float) FMath::Abs( FMath::FindDeltaAngleDegrees( LastRoomSpaceHeadToWorld.GetRotation().Rotator().Pitch, RoomSpaceHeadToWorld.GetRotation().Rotator().Pitch ) ) / DeltaTime;
		const float RollVelocity = (float) FMath::Abs( FMath::FindDeltaAngleDegrees( LastRoomSpaceHeadToWorld.GetRotation().Rotator().Roll, RoomSpaceHeadToWorld.GetRotation().Rotator().Roll ) ) / DeltaTime;
		const float RotationVelocity = YawVelocity + PitchVelocity + RollVelocity;

		static float LastLocationVelocity = LocationVelocity;
		static float LastRotationVelocity = RotationVelocity;

		const float SmoothLocationVelocity = FMath::Lerp( LocationVelocity, LastLocationVelocity, VREd::HeadVelocitySmoothing->GetFloat() );
		const float SmoothRotationVelocity = FMath::Lerp( RotationVelocity, LastRotationVelocity, VREd::HeadVelocitySmoothing->GetFloat() );

		LastLocationVelocity = SmoothLocationVelocity;
		LastRotationVelocity = SmoothRotationVelocity;
		
		LastRoomSpaceHeadToWorld = RoomSpaceHeadToWorld;
		LastWorldScaleFactor = WorldScaleFactor;

		const float LocationVelocityAlpha = FMath::Clamp( SmoothLocationVelocity / MaxLocationVelocity, 0.0f, 1.0f );
		const float RotationVelocityAlpha = FMath::Clamp( SmoothRotationVelocity / MaxRotationVelocity, 0.0f, 1.0f );

		const FTransform HeadToWorld = WorldInteraction->GetHeadTransform();

		{
			FVector HeadLocationVelocityOffset = FVector::ZeroVector;
			HeadLocationVelocityOffset.InitFromString( VREd::HeadLocationVelocityOffset->GetString() );
			HeadLocationVelocityOffset *= WorldScaleFactor;

			const FColor Color = FColor::MakeFromColorTemperature( 6000.0f - LocationVelocityAlpha * 5000.0f );
			const float Thickness = FMath::Lerp( MinThickness, MaxThickness, LocationVelocityAlpha );
			const FTransform UIToHeadTransform = FTransform( FRotator( 0.0f, 0.0f, 0.0f ).Quaternion(), HeadLocationVelocityOffset );
			const FTransform UIToWorld = UIToHeadTransform * HeadToWorld;
			DrawDebug2DDonut( GetWorld(), UIToWorld.ToMatrixNoScale(), MinInnerRadius, FMath::Lerp( MinInnerRadius, MaxOuterRadius, LocationVelocityAlpha ), 64, Color, false, 0.0f, SDPG_World, Thickness );
		}

		{
			FVector HeadRotationVelocityOffset = FVector::ZeroVector;
			HeadRotationVelocityOffset.InitFromString( VREd::HeadRotationVelocityOffset->GetString() );
			HeadRotationVelocityOffset *= WorldScaleFactor;

			const FColor Color = FColor::MakeFromColorTemperature( 6000.0f - RotationVelocityAlpha * 5000.0f );
			const float Thickness = FMath::Lerp( MinThickness, MaxThickness, RotationVelocityAlpha );
			const FTransform UIToHeadTransform = FTransform( FRotator( 0.0f, 0.0f, 0.0f ).Quaternion(), HeadRotationVelocityOffset );
			const FTransform UIToWorld = UIToHeadTransform * HeadToWorld;
			DrawDebug2DDonut( GetWorld(), UIToWorld.ToMatrixNoScale(), MinInnerRadius, FMath::Lerp( MinInnerRadius, MaxOuterRadius, RotationVelocityAlpha ), 64, Color, false, 0.0f, SDPG_World, Thickness );
		}
	}

	bFirstTick = false;
}

bool UVREditorMode::GetLaserForHand(EControllerHand InHand, FVector& OutLaserStart, FVector& OutLaserEnd) const
{
	if (UVREditorInteractor* Interactor = GetHandInteractor(InHand))
	{
		AVREditorTeleporter* Teleporter = Interactor->GetTeleportActor();

		const bool bHasLaser =
			Interactor->GetControllerType() == EControllerType::AssistingLaser
			|| Interactor->GetControllerType() == EControllerType::Laser
			|| (Teleporter && Teleporter->IsAiming());

		if (bHasLaser)
		{
			OutLaserStart = Interactor->GetLaserStart();
			OutLaserEnd = Interactor->GetLaserEnd();
			return true;
		}
	}

	return false;
}

FTransform UVREditorMode::GetRoomTransform() const
{
	return WorldInteraction->GetRoomTransform();
}

void UVREditorMode::SetRoomTransform( const FTransform& NewRoomTransform )
{
	WorldInteraction->SetRoomTransform( NewRoomTransform );
}

FTransform UVREditorMode::GetRoomSpaceHeadTransform() const
{
	return WorldInteraction->GetRoomSpaceHeadTransform();
}

FTransform UVREditorMode::GetHeadTransform() const
{
	return WorldInteraction->GetHeadTransform();
}

const UViewportWorldInteraction& UVREditorMode::GetWorldInteraction() const
{
	return *WorldInteraction;
}

UViewportWorldInteraction& UVREditorMode::GetWorldInteraction()
{
	return *WorldInteraction;
}

bool UVREditorMode::IsFullyInitialized() const
{
	return bIsFullyInitialized;
}

bool UVREditorMode::IsShowingRadialMenu(const UVREditorInteractor* Interactor) const
{
	return UISystem->IsShowingRadialMenu(Interactor);
}

void UVREditorMode::SetGameView(bool bGameView)
{
	if (TSharedPtr<SLevelViewport> Viewport = VREditorLevelViewportWeakPtr.Pin())
	{
		// We can't actually set the viewport to game view, because AVREditorAvatarActor::IsEditorOnly is
		// overridden to return true, and so its owned components (including the interactors) get hidden.
		// However, clearing the "editor" flag turns out to get us close to what we'd want.
		FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
		ViewportClient.EngineShowFlags.SetEditor(!bGameView);
	}
}

bool UVREditorMode::IsInGameView() const
{
	if (TSharedPtr<SLevelViewport> Viewport = VREditorLevelViewportWeakPtr.Pin())
	{
		FLevelEditorViewportClient& ViewportClient = Viewport->GetLevelViewportClient();
		return ViewportClient.EngineShowFlags.Editor == 0;
	}

	return false;
}

float UVREditorMode::GetWorldScaleFactor() const
{
	return WorldInteraction->GetWorldScaleFactor();
}

void UVREditorMode::ToggleFlashlight( UVREditorInteractor* Interactor )
{
	UVREditorInteractor* MotionControllerInteractor = Cast<UVREditorInteractor>( Interactor );
	if ( MotionControllerInteractor )
	{
		if ( FlashlightComponent == nullptr )
		{
			FlashlightComponent = NewObject<USpotLightComponent>( AvatarActor );
			AvatarActor->AddOwnedComponent( FlashlightComponent );
			FlashlightComponent->RegisterComponent();
			FlashlightComponent->SetMobility( EComponentMobility::Movable );
			FlashlightComponent->SetCastShadows( false );
			FlashlightComponent->bUseInverseSquaredFalloff = false;
			//@todo vreditor tweak
			FlashlightComponent->SetLightFalloffExponent( 8.0f );
			FlashlightComponent->SetIntensity( 20.0f );
			FlashlightComponent->SetOuterConeAngle( 25.0f );
			FlashlightComponent->SetInnerConeAngle( 25.0f );

		}

		const FAttachmentTransformRules AttachmentTransformRules = FAttachmentTransformRules( EAttachmentRule::KeepRelative, true );
		FlashlightComponent->AttachToComponent( MotionControllerInteractor->GetMotionControllerComponent(), AttachmentTransformRules );
		bIsFlashlightOn = !bIsFlashlightOn;
		FlashlightComponent->SetVisibility( bIsFlashlightOn );
	}
}

void UVREditorMode::CycleTransformGizmoHandleType()
{
	EGizmoHandleTypes NewGizmoType = (EGizmoHandleTypes)( (uint8)WorldInteraction->GetCurrentGizmoType() + 1 );

	if( NewGizmoType > EGizmoHandleTypes::Scale )
	{
		NewGizmoType = EGizmoHandleTypes::All;
	}

	WorldInteraction->SetGizmoHandleType( NewGizmoType );
}

FName UVREditorMode::GetHMDDeviceType() const
{
	if (HMDDeviceTypeOverride != NAME_None)
	{
		return HMDDeviceTypeOverride;
	}

	return GEngine->XRSystem.IsValid() ? GEngine->XRSystem->GetSystemName() : FName();
}

FLinearColor UVREditorMode::GetColor( const EColors Color ) const
{
	return Colors[ (int32)Color ];
}

float UVREditorMode::GetDefaultVRNearClipPlane() const
{
	return VREd::DefaultVRNearClipPlane->GetFloat();
}

void UVREditorMode::RefreshVREditorSequencer(class ISequencer* InCurrentSequencer)
{
	CurrentSequencer = InCurrentSequencer;
	// Tell the VR Editor UI system to refresh the Sequencer UI
	if (bActuallyUsingVR && UISystem != nullptr)
	{
		GetUISystem().UpdateSequencerUI();
	}
}

void UVREditorMode::RefreshActorPreviewWidget(TSharedRef<SWidget> InWidget, int32 Index, AActor *Actor, bool bIsPanelDetached)
{
	if (bActuallyUsingVR && UISystem != nullptr)
	{
		if (bIsPanelDetached)
		{
			GetUISystem().UpdateDetachedActorPreviewUI(InWidget, Index);
		}
		else
		{
			GetUISystem().UpdateActorPreviewUI(InWidget, Index, Actor);
		}
		
	}
}

void UVREditorMode::UpdateExternalUMGUI(const FVREditorFloatingUICreationContext& CreationContext) 
{
	if (bActuallyUsingVR && UISystem != nullptr)
	{
		GetUISystem().UpdateExternalUMGUI(CreationContext); 
	}
}

void UVREditorMode::UpdateExternalSlateUI(TSharedRef<SWidget> InWidget, FName Name, FVector2D InSize)
{
	if (bActuallyUsingVR && UISystem != nullptr)
	{
		GetUISystem().UpdateExternalSlateUI(InWidget, Name, InSize);
	}
}

class ISequencer* UVREditorMode::GetCurrentSequencer()
{
	return CurrentSequencer;
}

bool UVREditorMode::IsHandAimingTowardsCapsule(UViewportInteractor* Interactor, const FTransform& CapsuleTransform, FVector CapsuleStart, const FVector CapsuleEnd, const float CapsuleRadius, const float MinDistanceToCapsule, const FVector CapsuleFrontDirection, const float MinDotForAimingAtCapsule) const
{
	bool bIsAimingTowards = false;
	const float WorldScaleFactor = GetWorldScaleFactor();

	FVector LaserPointerStart, LaserPointerEnd;
	if( Interactor->GetLaserPointer( /* Out */ LaserPointerStart, /* Out */ LaserPointerEnd ) )
	{
		const FVector LaserPointerStartInCapsuleSpace = CapsuleTransform.InverseTransformPosition( LaserPointerStart );
		const FVector LaserPointerEndInCapsuleSpace = CapsuleTransform.InverseTransformPosition( LaserPointerEnd );

		FVector ClosestPointOnLaserPointer, ClosestPointOnUICapsule;
		FMath::SegmentDistToSegment(
			LaserPointerStartInCapsuleSpace, LaserPointerEndInCapsuleSpace,
			CapsuleStart, CapsuleEnd,
			/* Out */ ClosestPointOnLaserPointer,
			/* Out */ ClosestPointOnUICapsule );

		const bool bIsClosestPointInsideCapsule = ( ClosestPointOnLaserPointer - ClosestPointOnUICapsule ).Size() <= CapsuleRadius;

		const FVector TowardLaserPointerVector = ( ClosestPointOnLaserPointer - ClosestPointOnUICapsule ).GetSafeNormal();

		// Apply capsule radius
		ClosestPointOnUICapsule += TowardLaserPointerVector * CapsuleRadius;

		#if 0	// @todo vreditor debug
		{
			const float RenderCapsuleLength = (float) ( CapsuleEnd - CapsuleStart ).Size() + CapsuleRadius * 2.0f;
			// @todo vreditor:  This capsule draws with the wrong orientation
			#if 0
			{
				DrawDebugCapsule( GetWorld(), CapsuleTransform.TransformPosition( CapsuleStart + ( CapsuleEnd - CapsuleStart ) * 0.5f ), RenderCapsuleLength * 0.5f, CapsuleRadius, CapsuleTransform.GetRotation() * FRotator( 90.0f, 0, 0 ).Quaternion(), FColor::Green, false, 0.0f );
			}
			#endif
			DrawDebugLine( GetWorld(), CapsuleTransform.TransformPosition( ClosestPointOnLaserPointer ), CapsuleTransform.TransformPosition( ClosestPointOnUICapsule ), FColor::Green, false, 0.0f );
			DrawDebugSphere( GetWorld(), CapsuleTransform.TransformPosition( ClosestPointOnLaserPointer ), 1.5f * WorldScaleFactor, 32, FColor::Red, false, 0.0f );
			DrawDebugSphere( GetWorld(), CapsuleTransform.TransformPosition( ClosestPointOnUICapsule ), 1.5f * WorldScaleFactor, 32, FColor::Green, false, 0.0f );
		}
		#endif

		// If we're really close to the capsule
		if( bIsClosestPointInsideCapsule ||
			( ClosestPointOnUICapsule - ClosestPointOnLaserPointer ).Size() <= MinDistanceToCapsule )
		{
			const FVector LaserPointerDirectionInCapsuleSpace = ( LaserPointerEndInCapsuleSpace - LaserPointerStartInCapsuleSpace ).GetSafeNormal();

			#if 0	// @todo vreditor debug
			{
				DrawDebugLine( GetWorld(), CapsuleTransform.TransformPosition( FVector::ZeroVector ), CapsuleTransform.TransformPosition( CapsuleFrontDirection * 5.0f ), FColor::Yellow, false, 0.0f );
				DrawDebugLine( GetWorld(), CapsuleTransform.TransformPosition( FVector::ZeroVector ), CapsuleTransform.TransformPosition( -LaserPointerDirectionInCapsuleSpace * 5.0f ), FColor::Purple, false, 0.0f );
			}
			#endif

			const float Dot = (float) FVector::DotProduct( CapsuleFrontDirection, -LaserPointerDirectionInCapsuleSpace );
			if( Dot >= MinDotForAimingAtCapsule )
			{
				bIsAimingTowards = true;
			}
		}
	}

	return bIsAimingTowards;
}

UVREditorInteractor* UVREditorMode::GetHandInteractor( const EControllerHand ControllerHand ) const 
{
	for (UVREditorInteractor* Interactor : Interactors)
	{
		if (Interactor->GetControllerSide() == ControllerHand)
		{
			return Interactor;
		}
	}

	return nullptr;
}

void UVREditorMode::SnapSelectedActorsToGround()
{
	TSharedPtr<SLevelViewport> LevelEditorViewport = StaticCastSharedPtr<SLevelViewport>(WorldInteraction->GetDefaultOptionalViewportClient()->GetEditorViewportWidget());
	if (LevelEditorViewport.IsValid())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		const FLevelEditorCommands& Commands = LevelEditorModule.GetLevelEditorCommands();
		const TSharedPtr< FUICommandList >& CommandList = LevelEditorViewport->GetParentLevelEditor().Pin()->GetLevelEditorActions(); //@todo vreditor: Cast on leveleditor

		CommandList->ExecuteAction(Commands.SnapBottomCenterBoundsToFloor.ToSharedRef());

		// Force transformables to refresh
		GEditor->NoteSelectionChange();
	}
}

const UVREditorMode::FSavedEditorState& UVREditorMode::GetSavedEditorState() const
{
	const FSavedEditorState& SavedEditorState = static_cast<const FSavedEditorState&>(SavedEditorStateChecked());
	return SavedEditorState;
}

void UVREditorMode::SaveSequencerSettings(bool bInKeyAllEnabled, EAutoChangeMode InAutoChangeMode, const class USequencerSettings& InSequencerSettings)
{
	FSavedEditorState& SavedEditorState = static_cast<FSavedEditorState&>(SavedEditorStateChecked());
	SavedEditorState.bKeyAllEnabled = bInKeyAllEnabled;
	SavedEditorState.AutoChangeMode = InAutoChangeMode;
}

void UVREditorMode::TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState)
{
	Super::TransitionWorld(NewWorld, TransitionState);

	UISystem->TransitionWorld(NewWorld, TransitionState);
}


void UVREditorMode::RestoreWorldToMeters()
{
	const FSavedEditorState& SavedEditorState = static_cast<const FSavedEditorState&>(SavedEditorStateChecked());

	const float DefaultWorldToMeters = VREd::DefaultWorldToMeters->GetFloat();
	GetWorld()->GetWorldSettings()->WorldToMeters = DefaultWorldToMeters != 0.0f ? DefaultWorldToMeters : SavedEditorState.WorldToMetersScale;
	ENGINE_API extern float GNewWorldToMetersScale;
	GNewWorldToMetersScale = 0.0f;
}

UStaticMeshComponent* UVREditorMode::CreateMotionControllerMesh(AActor* OwningActor, USceneComponent* AttachmentToComponent, UStaticMesh* OptionalControllerMesh)
{	
	UStaticMesh* ControllerMesh = OptionalControllerMesh;

	if (ControllerMesh == nullptr)
	{
		if (GetHMDDeviceType() == FName(TEXT("SteamVR")))
		{
			ControllerMesh = AssetContainer->VivePreControllerMesh;
		}
		else if (GetHMDDeviceType() == FName(TEXT("OculusHMD")))
		{
			ControllerMesh = AssetContainer->OculusControllerMesh;
		}
		else
		{
			ControllerMesh = AssetContainer->GenericControllerMesh;
		}
	}

	return CreateMesh(OwningActor, ControllerMesh, AttachmentToComponent);
}

UStaticMeshComponent* UVREditorMode::CreateMesh( AActor* OwningActor, const FString& MeshName, USceneComponent* AttachmentToComponent /*= nullptr */ )
{
	UStaticMesh* Mesh = LoadObject<UStaticMesh>(nullptr, *MeshName);
	check(Mesh != nullptr);
	return CreateMesh(OwningActor, Mesh, AttachmentToComponent);
}

UStaticMeshComponent* UVREditorMode::CreateMesh(AActor* OwningActor, UStaticMesh* Mesh, USceneComponent* AttachmentToComponent /*= nullptr */)
{
	UStaticMeshComponent* CreatedMeshComponent = NewObject<UStaticMeshComponent>(OwningActor);
	OwningActor->AddOwnedComponent(CreatedMeshComponent);
	if (AttachmentToComponent != nullptr)
	{
		CreatedMeshComponent->SetupAttachment(AttachmentToComponent);
	}

	CreatedMeshComponent->RegisterComponent();

	CreatedMeshComponent->SetStaticMesh(Mesh);
	CreatedMeshComponent->SetMobility(EComponentMobility::Movable);
	CreatedMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	CreatedMeshComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);
	return CreatedMeshComponent;
}

const UVREditorAssetContainer& UVREditorMode::GetAssetContainer() const
{
	return *AssetContainer;
}

UVREditorAssetContainer& UVREditorMode::LoadAssetContainer()
{
	UVREditorAssetContainer* AssetContainer = LoadObject<UVREditorAssetContainer>(nullptr, UVREditorMode::AssetContainerPath);
	checkf(AssetContainer, TEXT("Failed to load ViewportInteractionAssetContainer (%s). See log for reason."), UVREditorMode::AssetContainerPath);
	return *AssetContainer;
}

void UVREditorMode::PlaySound(USoundBase* SoundBase, const FVector& InWorldLocation, const float InVolume /*= 1.0f*/)
{
	if (IsActive() && bIsFullyInitialized && GEditor != nullptr && GEditor->CanPlayEditorSound())
	{
		const float Volume = InVolume*VREd::SFXMultiplier->GetFloat();
		UGameplayStatics::PlaySoundAtLocation(GetWorld(), SoundBase, InWorldLocation, Volume);
	}
}

bool UVREditorMode::IsAimingTeleport() const
{
	return TeleportActor->IsAiming();
}

// static
void UVREditorMode::ToggleDebugMode()
{
	UVREditorMode::bDebugModeEnabled = !UVREditorMode::bDebugModeEnabled;
	IVREditorModule& VREditorModule = IVREditorModule::Get();
	UVREditorMode* VRMode = VREditorModule.GetVRMode();
	if (VRMode != nullptr)
	{
		VRMode->OnToggleDebugMode().Broadcast(UVREditorMode::bDebugModeEnabled);
	}
}

// static
bool UVREditorMode::IsDebugModeEnabled()
{
	return UVREditorMode::bDebugModeEnabled;
}

class AVREditorTeleporter* UVREditorMode::GetTeleportActor()
{
	return TeleportActor;
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
