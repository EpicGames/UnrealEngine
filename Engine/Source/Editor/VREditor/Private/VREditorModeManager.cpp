// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorModeManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "InputCoreTypes.h"
#include "VREditorMode.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/PlayerInput.h"
#include "Editor.h"
#include "HAL/PlatformApplicationMisc.h"

#include "EngineAnalytics.h"
#include "EngineGlobals.h"
#include "LevelEditor.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "EditorWorldExtension.h"
#include "ViewportWorldInteraction.h"
#include "VRModeSettings.h"
#include "Dialogs/Dialogs.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "UnrealEdMisc.h"
#include "Settings/EditorStyleSettings.h"
#include "VREditorInteractor.h"
#include "SLevelViewport.h"

#define LOCTEXT_NAMESPACE "VREditor"

FVREditorModeManager::FVREditorModeManager() :
	CurrentVREditorMode( nullptr ),
	bEnableVRRequest( false ),
	HMDWornState( EHMDWornState::Unknown ), 
	bAddedViewportWorldInteractionExtension( false )
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.OnFilesLoaded().AddRaw(this, &FVREditorModeManager::HandleAssetFilesLoaded);
	}
	else
	{
		HandleAssetFilesLoaded();
	}

	FEditorDelegates::OnEditorPreExit.AddRaw(this, &FVREditorModeManager::HandleEditorPreExit);
}

FVREditorModeManager::~FVREditorModeManager()
{
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(AssetRegistryConstants::ModuleName))
	{
		if (IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet())
		{
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}
	}

	if (IsValid(CurrentVREditorMode))
	{
		CurrentVREditorMode->OnVRModeEntryComplete().RemoveAll(this);
		CurrentVREditorMode = nullptr;
	}

	FEditorDelegates::OnEditorPreExit.RemoveAll(this);
}

void FVREditorModeManager::Tick( const float DeltaTime )
{
	// You can only auto-enter VR if the setting is enabled. Other criteria are that the VR Editor is enabled in experimental settings, that you are not in PIE, and that the editor is foreground.
	IHeadMountedDisplay * const HMD = GEngine != nullptr && GEngine->XRSystem.IsValid() ? GEngine->XRSystem->GetHMDDevice() : nullptr;
	if (GetDefault<UVRModeSettings>()->bEnableAutoVREditMode
		&& HMD
		&& FPlatformApplicationMisc::IsThisApplicationForeground())
	{
		const EHMDWornState::Type LatestHMDWornState = HMD->GetHMDWornState();
		if (HMDWornState != LatestHMDWornState)
		{
			HMDWornState = LatestHMDWornState;
			if (HMDWornState == EHMDWornState::Worn && CurrentVREditorMode == nullptr)
			{
				EnableVREditor( true, false );
			}
			else if (HMDWornState == EHMDWornState::NotWorn && CurrentVREditorMode != nullptr)
			{
				EnableVREditor( false, false );
			}
		}
	}

	if(CurrentVREditorMode != nullptr && CurrentVREditorMode->WantsToExitMode())
	{
		// For a standard exit, also take the HMD out of stereo mode
		const bool bShouldDisableStereo = true;
		CloseVREditor( bShouldDisableStereo );
	}

	// Start the VR Editor mode
	if (bEnableVRRequest)
	{
		EnableVREditor(true, false);
		bEnableVRRequest = false;
	}
}

bool FVREditorModeManager::IsTickable() const
{
	const FProjectDescriptor* CurrentProject = IProjectManager::Get().GetCurrentProject();
	return CurrentProject != nullptr;
}

void FVREditorModeManager::EnableVREditor( const bool bEnable, const bool bForceWithoutHMD )
{
	// Don't do anything when the current VR Editor is already in the requested state
	const bool bIsEnabled = IsValid(CurrentVREditorMode);
	if (bEnable != bIsEnabled)
	{
		if( bEnable && ( IsVREditorAvailable() || bForceWithoutHMD ))
		{
			StartVREditorMode(bForceWithoutHMD);
		}
		else if( !bEnable )
		{
			// For a standard exit, take the HMD out of stereo mode
			const bool bShouldDisableStereo = true;
			CloseVREditor( bShouldDisableStereo );
		}
	}
}

bool FVREditorModeManager::IsVREditorActive() const
{
	return CurrentVREditorMode != nullptr && CurrentVREditorMode->IsActive();
}

bool FVREditorModeManager::IsVREditorAvailable() const
{
	if (!GetDefault<UVRModeSettings>()->ModeClass.LoadSynchronous())
	{
		return false;
	}

	if (GEditor->IsPlayingSessionInEditor())
	{
		return false;
	}

	if (!GEngine->XRSystem.IsValid())
	{
		return false;
	}

	const TSharedRef<ILevelEditor>& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor").GetFirstLevelEditor().ToSharedRef();
	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
	if (!ActiveLevelViewport.IsValid())
	{
		return false;
	}

	return GEngine->XRSystem->GetHMDDevice() && GEngine->XRSystem->GetHMDDevice()->IsHMDConnected();
}

bool FVREditorModeManager::IsVREditorButtonActive() const
{
	const bool bAnyModeAvailable = CachedModeClassPaths.Num() > 0;
	return bAnyModeAvailable;
}


UVREditorModeBase* FVREditorModeManager::GetCurrentVREditorMode()
{
	return CurrentVREditorMode;
}

void FVREditorModeManager::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( CurrentVREditorMode );
}

void FVREditorModeManager::HandleModeEntryComplete()
{
	// Connects the mode UObject's event to the module delegate.
	if (CurrentVREditorMode->IsActuallyUsingVR())
	{
		OnVREditingModeEnterHandle.Broadcast();
	}
}

void FVREditorModeManager::StartVREditorMode( const bool bForceWithoutHMD )
{
	if (!IsEngineExitRequested())
	{
		UVREditorModeBase* VRMode = nullptr;
		{
			UWorld* World = GEditor->bIsSimulatingInEditor ? GEditor->PlayWorld : GWorld;
			UEditorWorldExtensionCollection* ExtensionCollection = GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(World);
			check(ExtensionCollection != nullptr);

			// Create vr editor mode.
			const TSoftClassPtr<UVREditorModeBase> ModeClassSoft = GetDefault<UVRModeSettings>()->ModeClass;
			ModeClassSoft.LoadSynchronous();
			check(ModeClassSoft.IsValid());

			VRMode = NewObject<UVREditorModeBase>(World, ModeClassSoft.Get());
			check(VRMode != nullptr);
			VRMode->OnVRModeEntryComplete().AddRaw(this, &FVREditorModeManager::HandleModeEntryComplete);
			ExtensionCollection->AddExtension(VRMode);
			CurrentVREditorMode = VRMode;
		}

		// Tell the level editor we want to be notified when selection changes
		{
			FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
			LevelEditor.OnMapChanged().AddRaw( this, &FVREditorModeManager::OnMapChanged );
		}

		VRMode->SetActuallyUsingVR( !bForceWithoutHMD );
		VRMode->Enter();
		// Note: Enter() may, in the case of early init failure, re-enter CloseVREditor,
		// causing CurrentVREditorMode to be nullified/invalidated beyond this point

		if (FEngineAnalytics::IsAvailable())
		{
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.InitVREditorMode"), FAnalyticsEventAttribute(TEXT("Enterprise"), IProjectManager::Get().IsEnterpriseProject()));

			// Take note of VREditor entering (only if actually in VR)
			if (CurrentVREditorMode && CurrentVREditorMode->IsActuallyUsingVR())
			{
				TArray<FAnalyticsEventAttribute> Attributes;
				FString HMDName = GEditor->XRSystem->GetSystemName().ToString();
				Attributes.Add(FAnalyticsEventAttribute(TEXT("HMDDevice"), HMDName));
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.EnterVRMode"), Attributes);
			}
		}
	}
}

void FVREditorModeManager::CloseVREditor( const bool bShouldDisableStereo )
{
	FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>( "LevelEditor" );
	if ( LevelEditor != nullptr )
	{
		LevelEditor->OnMapChanged().RemoveAll( this );
	}

	if ( CurrentVREditorMode != nullptr )
	{
		CurrentVREditorMode->Exit( bShouldDisableStereo );

		UEditorWorldExtensionCollection* Collection = CurrentVREditorMode->GetOwningCollection();
		check(Collection != nullptr);
		Collection->RemoveExtension(CurrentVREditorMode);

		if (CurrentVREditorMode->IsActuallyUsingVR())
		{
			// Take note of VREditor exiting (only if actually in VR)
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.ExitVRMode"));
			}

			OnVREditingModeExitHandle.Broadcast();
		}

		CurrentVREditorMode->MarkAsGarbage();
		CurrentVREditorMode = nullptr;
	}
}

void FVREditorModeManager::SetDirectWorldToMeters( const float NewWorldToMeters )
{
	GWorld->GetWorldSettings()->WorldToMeters = NewWorldToMeters; //@todo VREditor: Do not use GWorld
	ENGINE_API extern float GNewWorldToMetersScale;
	GNewWorldToMetersScale = 0.0f;
}

void FVREditorModeManager::OnMapChanged( UWorld* World, EMapChangeType MapChangeType )
{
	if( CurrentVREditorMode && CurrentVREditorMode->IsActive() )
	{
		// When changing maps, we are going to close VR editor mode but then reopen it, so don't take the HMD out of stereo mode
		const bool bShouldDisableStereo = false;
		CloseVREditor( bShouldDisableStereo );
		if (MapChangeType != EMapChangeType::SaveMap)
		{
			bEnableVRRequest = true;
		}
	}
	CurrentVREditorMode = nullptr;
}

void FVREditorModeManager::GetConcreteModeClasses(TArray<UClass*>& OutModeClasses) const
{
	for (const FTopLevelAssetPath& ClassPath : CachedModeClassPaths)
	{
		if (UClass* Class = TryGetConcreteModeClass(ClassPath))
		{
			OutModeClasses.Add(Class);
		}
	}
}

UClass* FVREditorModeManager::TryGetConcreteModeClass(const FTopLevelAssetPath& ClassPath) const
{
	TSoftClassPtr<UVREditorModeBase> SoftClass = TSoftClassPtr<UVREditorModeBase>(FSoftObjectPath(ClassPath));
	if (UClass* Class = SoftClass.LoadSynchronous())
	{
		if (!Class->HasAnyClassFlags(CLASS_Abstract) && (Class->GetAuthoritativeClass() == Class))
		{
			return Class;
		}
	}

	return nullptr;
}

void FVREditorModeManager::GetModeClassPaths(TSet<FTopLevelAssetPath>& OutModeClasses) const
{
	TArray<FTopLevelAssetPath> BaseModeClasses;
	BaseModeClasses.Add(UVREditorModeBase::StaticClass()->GetClassPathName());

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	AssetRegistry.GetDerivedClassNames(BaseModeClasses, TSet<FTopLevelAssetPath>(), OutModeClasses);
}

void FVREditorModeManager::UpdateCachedModeClassPaths()
{
	TSet<FTopLevelAssetPath> UpdatedModeClassPaths;
	GetModeClassPaths(UpdatedModeClassPaths);

	// Exclude any ineligible (abstract, failed to load, etc) modes
	for (TSet<FTopLevelAssetPath>::TIterator It = UpdatedModeClassPaths.CreateIterator(); It; ++It)
	{
		if (!TryGetConcreteModeClass(*It))
		{
			It.RemoveCurrent();
		}
	}

	// Issue update events.
	TSet<FTopLevelAssetPath> Union = CachedModeClassPaths.Union(UpdatedModeClassPaths);
	for (const FTopLevelAssetPath& Path : Union)
	{
		if (!CachedModeClassPaths.Contains(Path))
		{
			OnModeClassAdded.Broadcast(Path);
		}
		else if (!UpdatedModeClassPaths.Contains(Path))
		{
			OnModeClassRemoved.Broadcast(Path);
		}
	}

	CachedModeClassPaths = MoveTemp(UpdatedModeClassPaths);
}

void FVREditorModeManager::HandleAssetFilesLoaded()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	AssetRegistry.OnFilesLoaded().RemoveAll(this);
	AssetRegistry.OnAssetsAdded().AddRaw(this, &FVREditorModeManager::HandleAssetAddedOrRemoved);
	AssetRegistry.OnAssetsRemoved().AddRaw(this, &FVREditorModeManager::HandleAssetAddedOrRemoved);
	UpdateCachedModeClassPaths();
}

void FVREditorModeManager::HandleAssetAddedOrRemoved(TConstArrayView<FAssetData> NewAssetData)
{
	UpdateCachedModeClassPaths();
}

void FVREditorModeManager::HandleEditorPreExit()
{
	if (IsVREditorActive())
	{
		const bool bEnable_false = false;
		const bool bForceWithoutHMD_false = false;
		EnableVREditor(bEnable_false, bForceWithoutHMD_false);
	}
}

#undef LOCTEXT_NAMESPACE
