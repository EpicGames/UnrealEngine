// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/GCObject.h"
#include "TickableEditorObject.h"
#include "HeadMountedDisplayTypes.h"
#include "IVREditorModule.h"


class UVREditorModeBase;
enum class EMapChangeType : uint8;


/**
 * Manages starting and closing the VREditor mode
 */
class FVREditorModeManager : public FGCObject, public FTickableEditorObject
{
public:
	/** Default constructor */
	FVREditorModeManager();

	/** Default destructor */
	~FVREditorModeManager();

	// FTickableEditorObject overrides
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FVREditorModeManager, STATGROUP_Tickables);
	}

	/** Start or stop the VR Editor */
	void EnableVREditor( const bool bEnable, const bool bForceWithoutHMD );

	/** If the VR Editor is currently running */
	bool IsVREditorActive() const;

	/** If the VR Editor is currently available */
	bool IsVREditorAvailable() const;

	/** If the VR Editor button active */
	bool IsVREditorButtonActive() const;

	/** Gets the current VR Editor mode that was enabled */
	UVREditorModeBase* GetCurrentVREditorMode();

	// FGCObject
	virtual void AddReferencedObjects( FReferenceCollector& Collector );
	virtual FString GetReferencerName() const override
	{
		return TEXT("FVREditorModeManager");
	}
	// End FGCObject

	/** Return a multicast delegate which is executed when VR mode starts. */
	FOnVREditingModeEnter& OnVREditingModeEnter() { return OnVREditingModeEnterHandle; }

	/** Return a multicast delegate which is executed when VR mode stops. */
	FOnVREditingModeExit& OnVREditingModeExit() { return OnVREditingModeExitHandle; }

	/**
	 * Loads each cached UVREditorModeBase-derived class path, checking and appending
	 * only those which are suitable for use (not abstract, etc.) to the provided array. */
	void GetConcreteModeClasses(TArray<UClass*>& OutModeClasses) const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModeClassAdded, const FTopLevelAssetPath&);
	FModeClassAdded OnModeClassAdded;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModeClassRemoved, const FTopLevelAssetPath&);
	FModeClassRemoved OnModeClassRemoved;

private:
	UClass* TryGetConcreteModeClass(const FTopLevelAssetPath& ClassPath) const;

	/** Queries the asset registry for asset paths of all classes derived from UVREditorModeBase. */
	void GetModeClassPaths(TSet<FTopLevelAssetPath>& OutModeClassPaths) const;

	/** Stores the results of GetModeClassPaths() into CachedModeClassPaths. */
	void UpdateCachedModeClassPaths();

	/** Cached UVREditorModeBase-derived assets, updated in response to asset registry events. */
	TSet<FTopLevelAssetPath> CachedModeClassPaths;

	void HandleAssetFilesLoaded();
	void HandleAssetAddedOrRemoved(TConstArrayView<FAssetData> NewAssetData);

	/** Broadcasts when VR mode is started */
	FOnVREditingModeEnter OnVREditingModeEnterHandle;

	/** Broadcasts when VR mode is stopped */
	FOnVREditingModeExit OnVREditingModeExitHandle;

	void HandleModeEntryComplete();

	/** Saves the WorldToMeters and enters the mode belonging to GWorld */
	void StartVREditorMode( const bool bForceWithoutHMD );

	/** Closes the current VR Editor if any and sets the WorldToMeters to back to the one from before entering the VR mode */
	void CloseVREditor(const bool bShouldDisableStereo );

	/** Directly set the GWorld WorldToMeters */
	void SetDirectWorldToMeters( const float NewWorldToMeters );
	
	/** On level changed */
	void OnMapChanged( UWorld* World, EMapChangeType MapChangeType );

	/** Ensure we shut down cleanly early in editor shutdown. */
	void HandleEditorPreExit();
	
	/** The current mode, nullptr if none */
	UPROPERTY()
	TObjectPtr<UVREditorModeBase> CurrentVREditorMode;

	/** If the VR Editor mode needs to be enabled next tick */
	bool bEnableVRRequest;

	/** True when we detect that the user is wearing the HMD */
	EHMDWornState::Type HMDWornState;

	/** True if the ViewportWorldInteraction extension was not pre-existing. */
	bool bAddedViewportWorldInteractionExtension;
};
