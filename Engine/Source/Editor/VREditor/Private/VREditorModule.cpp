// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorModule.h"
#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "IHeadMountedDisplay.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "IVREditorModule.h"
#include "IXRTrackingSystem.h"
#include "LevelEditorActions.h"
#include "ToolMenus.h"
#include "VREditorModeManager.h"
#include "VREditorStyle.h"
#include "VREditorMode.h"
#include "VRModeSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HeadMountedDisplayTypes.h"
#include "UI/VREditorFloatingUI.h"


#define LOCTEXT_NAMESPACE "VREditorModule"


DEFINE_LOG_CATEGORY(LogVREditor);


class FVREditorModule : public IVREditorModule
{
public:
	FVREditorModule()
	{
	}

	// FModuleInterface overrides
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void PostLoadCallback() override;
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	// IVREditorModule overrides
	virtual bool IsVREditorEnabled() const override;
	virtual bool IsVREditorAvailable() const override;
	virtual bool IsVREditorButtonActive() const override;
	virtual void EnableVREditor( const bool bEnable, const bool bForceWithoutHMD ) override;
	virtual bool IsVREditorModeActive() override;
	virtual UVREditorModeBase* GetVRModeBase() override;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual UVREditorMode* GetVRMode() override;
	virtual void UpdateExternalUMGUI(const FVREditorFloatingUICreationContext& CreationContext) override;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual void UpdateActorPreview(TSharedRef<SWidget> InWidget, int32 Index, AActor *Actor, bool bIsPanelDetached = false) override;
	virtual void UpdateExternalSlateUI(TSharedRef<SWidget> InSlateWidget, FName Name, FVector2D InSize) override;
	virtual TSharedPtr<FExtender> GetRadialMenuExtender() override
	{
		return RadialMenuExtender;
	}


	static void ToggleForceVRMode();

	/** Return a multicast delegate which is executed when VR mode starts. */
	virtual FOnVREditingModeEnter& OnVREditingModeEnter() override { return ModeManager->OnVREditingModeEnter(); }

	/** Return a multicast delegate which is executed when VR mode stops. */
	virtual FOnVREditingModeExit& OnVREditingModeExit() override { return ModeManager->OnVREditingModeExit(); }

private:
	void ExtendToolbarMenu();
	void HandleModeClassAdded(const FTopLevelAssetPath& ClassPath);
	void AssignModeClassAndSave(const FTopLevelAssetPath& ClassPath);

private:
	TSharedPtr<class FExtender> RadialMenuExtender;

	/** Handles turning VR Editor mode on and off */
	TUniquePtr<FVREditorModeManager> ModeManager;

	FDelegateHandle DeferredInitDelegate;

	TWeakPtr<ISettingsSection> WeakSettingsSection;
};

namespace VREd
{
	static FAutoConsoleCommand ForceVRMode( TEXT( "VREd.ForceVRMode" ), TEXT( "Toggles VREditorMode, even if not in immersive VR" ), FConsoleCommandDelegate::CreateStatic( &FVREditorModule::ToggleForceVRMode ) );
}

void FVREditorModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("VREditor"));

	RadialMenuExtender = MakeShareable(new FExtender());

	ModeManager = MakeUnique<FVREditorModeManager>();
	ModeManager->OnModeClassAdded.AddRaw(this, &FVREditorModule::HandleModeClassAdded);

	// UToolMenus::RegisterStartupCallback is still too early.
	DeferredInitDelegate = FCoreDelegates::OnBeginFrame.AddLambda(
		[this]()
		{
			ExtendToolbarMenu();

			// Cache pointer to VR Mode settings section, registered elsewhere.
			ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");
			if (TSharedPtr<ISettingsContainer> EditorContainer = SettingsModule.GetContainer("Editor"))
			{
				if (TSharedPtr<ISettingsCategory> GeneralCategory = EditorContainer->GetCategory("General"))
				{
					WeakSettingsSection = GeneralCategory->GetSection("VR Mode");
				}
			}

			// Must happen last; this implicitly deallocates this lambda's captures.
			FCoreDelegates::OnBeginFrame.Remove(DeferredInitDelegate);
		}
	);
}

void FVREditorModule::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);

	if (GIsEditor)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FVREditorStyle::Shutdown();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	ModeManager.Reset();
}

void FVREditorModule::PostLoadCallback()
{
}

bool FVREditorModule::IsVREditorEnabled() const
{
	return ModeManager->IsVREditorActive();
}

bool FVREditorModule::IsVREditorAvailable() const
{
	return ModeManager->IsVREditorAvailable();
}

bool FVREditorModule::IsVREditorButtonActive() const
{
	return ModeManager->IsVREditorButtonActive();
}

void FVREditorModule::EnableVREditor( const bool bEnable, const bool bForceWithoutHMD )
{
	ModeManager->EnableVREditor( bEnable, bForceWithoutHMD );
}

bool FVREditorModule::IsVREditorModeActive()
{
	// Deprecated method only returns true for legacy UVREditorMode
	return GetVRMode() && ModeManager->IsVREditorActive();
}

UVREditorModeBase* FVREditorModule::GetVRModeBase()
{
	return ModeManager->GetCurrentVREditorMode();
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
UVREditorMode* FVREditorModule::GetVRMode()
{
	UVREditorModeBase* ModeBase = GetVRModeBase();
	return Cast<UVREditorMode>(ModeBase);
}

void FVREditorModule::UpdateActorPreview(TSharedRef<SWidget> InWidget, int32 Index, AActor* Actor, bool bIsPanelDetached)
{
	GetVRMode()->RefreshActorPreviewWidget(InWidget, Index, Actor, bIsPanelDetached);
}
  
void FVREditorModule::UpdateExternalUMGUI(const FVREditorFloatingUICreationContext& CreationContext)
{
	GetVRMode()->UpdateExternalUMGUI(CreationContext); 
}

void FVREditorModule::UpdateExternalSlateUI(TSharedRef<SWidget> InSlateWidget, FName Name, FVector2D InSize)
{
	GetVRMode()->UpdateExternalSlateUI(InSlateWidget, Name, InSize);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FVREditorModule::ToggleForceVRMode()
{
	const bool bForceWithoutHMD = true;
	FVREditorModule& Self = FModuleManager::GetModuleChecked< FVREditorModule >( TEXT( "VREditor" ) );
	Self.EnableVREditor( !Self.IsVREditorEnabled(), bForceWithoutHMD );
}

void FVREditorModule::ExtendToolbarMenu()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* EditorToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AssetsToolBar");
	if (!ensure(EditorToolbarMenu))
	{
		return;
	}

	FToolMenuSection* EditorToolbarSection = EditorToolbarMenu->FindSection("Content");
	if (!EditorToolbarSection)
	{
		return;
	}

	// The primary toggle button.
	EditorToolbarSection->AddEntry(FToolMenuEntry::InitToolBarButton(
		FLevelEditorCommands::Get().ToggleVR,
		LOCTEXT("ToggleVR", "VR Mode")
	));

	// The "..." split, which summons the tool menu.
	static const FName OptionsToolMenuName = "VrEditor.ToggleVrOptions";

	EditorToolbarSection->AddEntry(FToolMenuEntry::InitComboButton(
		"ToggleVrOptionsMenu",
		FUIAction(),
		FOnGetContent::CreateLambda([]() { return UToolMenus::Get()->GenerateWidget(OptionsToolMenuName, FToolMenuContext()); }),
		TAttribute<FText>(),
		LOCTEXT("ToggleVrOptions_ComboSplit_Tooltip", "Configure VR Editor Mode"),
		TAttribute<FSlateIcon>(),
		true
	));

	// Register the split options tool menu.
	UToolMenu* OptionsMenu = UToolMenus::Get()->RegisterMenu(OptionsToolMenuName);

	// The "Status" section indicates whether an HMD is available.
	FToolMenuSection& StatusSection = OptionsMenu->AddSection("Status");
	StatusSection.AddDynamicEntry("StatusEntry", FNewToolMenuSectionDelegate::CreateLambda(
		[](FToolMenuSection& InSection)
		{
			const bool bHasHMDDevice = GEngine->XRSystem
				&& GEngine->XRSystem->GetHMDDevice()
				&& GEngine->XRSystem->GetHMDDevice()->IsHMDConnected();

			const FText HmdAvailableText = LOCTEXT("ToggleVrOptions_Status_HmdAvailable", "Headset available ({XrVersionString})");
			const FText HmdNotAvailableText = LOCTEXT("ToggleVrOptions_Status_HmdNotAvailable", "No headset available");

			const FText StatusText = bHasHMDDevice
				? FText::FormatNamed(HmdAvailableText, TEXT("XrVersionString"), FText::FromString(GEngine->XRSystem->GetVersionString()))
				: HmdNotAvailableText;

			InSection.AddEntry(FToolMenuEntry::InitWidget(
				"HeadsetStatus",
				SNew(SBox)
				.Padding(FMargin(16.0f, 3.0f))
				[
					SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(StatusText)
				],
				FText::GetEmpty()
			));
		}));

	// The "Modes" section enumerates the UVREditorMode-derived classes for radio selection.
	OptionsMenu->AddDynamicSection("DynamicModesSection", FNewToolMenuDelegate::CreateLambda(
		[this](UToolMenu* InMenu)
		{
			FToolMenuSection& Section = InMenu->AddSection("Modes", LOCTEXT("ToggleVrOptions_ModesSection_Label", "Modes"));

			TSoftClassPtr<UVREditorModeBase> CurrentModeClassSoft = GetDefault<UVRModeSettings>()->ModeClass;
			UClass* CurrentModeClass = CurrentModeClassSoft.LoadSynchronous();

			TArray<UClass*> ModeClasses;
			ModeManager->GetConcreteModeClasses(ModeClasses);

			Algo::Sort(ModeClasses,
				[](const UClass* Lhs, const UClass* Rhs)
				{
					return Lhs->GetDisplayNameText().CompareTo(Rhs->GetDisplayNameText()) < 0;
				});

			if (!ModeClasses.Contains(CurrentModeClass))
			{
				FToolUIAction UnavailableModeAction;
				UnavailableModeAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([](const FToolMenuContext&) { return false; });
				UnavailableModeAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext&) { return ECheckBoxState::Checked; });

				Section.AddMenuEntry("UnavailableMode",
					CurrentModeClassSoft.IsNull() ? LOCTEXT("NullModeClass", "(Unset)") : FText::AsCultureInvariant(CurrentModeClassSoft.ToString()),
					TAttribute<FText>(),
					TAttribute<FSlateIcon>(),
					UnavailableModeAction,
					EUserInterfaceActionType::RadioButton);
			}

			for (UClass* Mode : ModeClasses)
			{
				FToolUIAction SelectModeAction;
				SelectModeAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
					[this, Mode]
					(const FToolMenuContext&)
					{
						AssignModeClassAndSave(Mode->GetClassPathName());
					});
				SelectModeAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(
					[](const FToolMenuContext&)
					{
						return true;
					});
				SelectModeAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
					[Mode](const FToolMenuContext&)
					{
						return Mode == GetDefault<UVRModeSettings>()->ModeClass.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					});

				Section.AddMenuEntry(Mode->GetFName(),
					Mode->GetDisplayNameText(),
					FText::AsCultureInvariant(Mode->GetClassPathName().ToString()),
					TAttribute<FSlateIcon>(),
					SelectModeAction,
					EUserInterfaceActionType::RadioButton);
			}
		}));

	FToolMenuSection& SettingsMenuSection = OptionsMenu->AddSection("Settings");

	SettingsMenuSection.AddSeparator("SettingsStartSeparator");

	SettingsMenuSection.AddMenuEntry(
		"EditorSettings",
		LOCTEXT("ToggleVrOptions_EditorSettings_Label", "Settings..."),
		LOCTEXT("ToggleVrOptions_EditorSettings_Tooltip", "Jump to the VR Editor Mode section in the Editor Preferences"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.GameSettings"),
		FUIAction(FExecuteAction::CreateLambda([]() { FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Editor", "General", "VR Mode"); }))
	);
}

void FVREditorModule::HandleModeClassAdded(const FTopLevelAssetPath& ClassPath)
{
	// If we don't have a mode class yet, assign the first one we discover.
	if (GetDefault<UVRModeSettings>()->ModeClass.IsNull())
	{
		AssignModeClassAndSave(ClassPath);
	}
}

void FVREditorModule::AssignModeClassAndSave(const FTopLevelAssetPath& ClassPath)
{
	UVRModeSettings* Settings = GetMutableDefault<UVRModeSettings>();
	Settings->ModeClass = FSoftObjectPath(ClassPath);

	if (FProperty* ModeClassProperty = FindFProperty<FProperty>(Settings->GetClass(),
		GET_MEMBER_NAME_CHECKED(UVRModeSettings, ModeClass)))
	{
		FPropertyChangedEvent PropertyUpdateStruct(ModeClassProperty, EPropertyChangeType::ValueSet);
		Settings->PostEditChangeProperty(PropertyUpdateStruct);
	}

	if (TSharedPtr<ISettingsSection> SettingsSection = WeakSettingsSection.Pin())
	{
		SettingsSection->Save();
	}
}


IMPLEMENT_MODULE( FVREditorModule, VREditor )


#undef LOCTEXT_NAMESPACE
