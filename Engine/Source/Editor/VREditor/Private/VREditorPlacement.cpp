// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorPlacement.h"
#include "HAL/IConsoleManager.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "VREditorMode.h"
#include "VREditorAssetContainer.h"
#include "ViewportInteractionTypes.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture.h"
#include "LevelEditorViewport.h"
#include "ViewportWorldInteraction.h"
#include "Engine/BrushBuilder.h"
#include "Components/PrimitiveComponent.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Sound/SoundCue.h"
#include "Editor.h"
#include "UI/VREditorUISystem.h"
#include "UI/VREditorFloatingUI.h"
#include "VREditorInteractor.h"

// For actor placement
#include "ObjectTools.h"
#include "AssetSelection.h"
#include "IPlacementModeModule.h"

#include "LevelEditor.h"
#include "SLevelViewport.h"
#include "LevelEditorActions.h"

#include "RenderCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorPlacement)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "VREditor"


namespace VREd
{
	static FAutoConsoleVariable HoverHapticFeedbackStrength( TEXT( "VREd.HoverHapticFeedbackStrength" ), 0.1f, TEXT( "Default strength for haptic feedback when hovering" ) );
	static FAutoConsoleVariable HoverHapticFeedbackTime( TEXT( "VREd.HoverHapticFeedbackTime" ), 0.2f, TEXT( "The minimum time between haptic feedback for hovering" ) );
	static FAutoConsoleVariable PivotPointTransformGizmo( TEXT( "VREd.PivotPointTransformGizmo" ), 1, TEXT( "If the pivot point transform gizmo is used instead of the bounding box gizmo" ) );
	static FAutoConsoleVariable DragHapticFeedbackStrength( TEXT( "VREd.DragHapticFeedbackStrength" ), 1.0f, TEXT( "Default strength for haptic feedback when starting to drag objects" ) );
}

UVREditorPlacement::UVREditorPlacement() : 
	Super(),
	VRMode( nullptr ),
	ViewportWorldInteraction(nullptr)
{
}

void UVREditorPlacement::Init(UVREditorMode* InVRMode)
{
	VRMode = InVRMode;
	ViewportWorldInteraction = &InVRMode->GetWorldInteraction();

	ViewportWorldInteraction->OnStopDragging().AddUObject( this, &UVREditorPlacement::StopDragging );
	ViewportWorldInteraction->OnWorldScaleChanged().AddUObject( this, &UVREditorPlacement::UpdateNearClipPlaneOnScaleChange );
}

void UVREditorPlacement::Shutdown()
{
	FEditorDelegates::OnAssetDragStarted.RemoveAll( this );
	ViewportWorldInteraction->OnStopDragging().RemoveAll( this );
	ViewportWorldInteraction->OnWorldScaleChanged().RemoveAll( this );

	PlacingMaterialOrTextureAsset = nullptr;
	FloatingUIAssetDraggedFrom = nullptr;
	VRMode = nullptr;
}

void UVREditorPlacement::StopDragging( UViewportInteractor* Interactor )
{
	if (FloatingUIAssetDraggedFrom != nullptr)
	{
		// If we were placing something, bring the window back
		const bool bShouldShow = true;
		const bool bSpawnInFront = false;
		const bool bDragFromOpen = false;
		const bool bPlaySound = false;
		VRMode->GetUISystem().ShowEditorUIPanel(FloatingUIAssetDraggedFrom, Cast<UVREditorInteractor>(Interactor->GetOtherInteractor()), bShouldShow, bSpawnInFront, bDragFromOpen, bPlaySound);
		FloatingUIAssetDraggedFrom = nullptr;
	}

	const EViewportInteractionDraggingMode InteractorDraggingMode = Interactor->GetDraggingMode();

	if (Interactor->GetDraggingMode() == EViewportInteractionDraggingMode::TransformablesFreely)
	{
		VRMode->OnPlacePreviewActor().Broadcast(false);
		GEditor->NoteSelectionChange();
		UVREditorInteractor* MotionController = Cast<UVREditorInteractor>(Interactor);
		if (VRMode->GetUISystem().GetUIInteractor() == MotionController)
		{
			MotionController->TryOverrideControllerType(EControllerType::Unknown);
		}
	}
}

void UVREditorPlacement::UpdateNearClipPlaneOnScaleChange(const float NewWorldToMetersScale)
{
	// Adjust the clipping plane for the user's scale, but don't let it be larger than the engine default
	const float DefaultWorldToMetersScale = VRMode->GetSavedEditorState().WorldToMetersScale;

	SetNearClipPlaneGlobals(FMath::Min((VRMode->GetDefaultVRNearClipPlane()) * (NewWorldToMetersScale / DefaultWorldToMetersScale), VRMode->GetSavedEditorState().NearClipPlane));
}


#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
