// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "AssetRegistry/AssetData.h"
#include "VREditorPlacement.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UActorComponent;
class UViewportInteractor;
class UViewportWorldInteraction;
class UVREditorMode;
struct FViewportActionKeyInput;

/**
 * VR Editor interaction with the 3D world
 */
UCLASS()
class UVREditorPlacement : public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UVREditorPlacement();

	/** Registers to events and sets initial values */
	void Init(UVREditorMode* InVRMode);
	
	/** Removes registered event */
	void Shutdown();


protected:
	/** When an interactor stops dragging */
	void StopDragging( UViewportInteractor* Interactor );

	/** When the world scale changes, update the near clip plane */
	void UpdateNearClipPlaneOnScaleChange(const float NewWorldToMetersScale);

protected:

	/** Owning object */
	UPROPERTY()
	TObjectPtr<UVREditorMode> VRMode;

	/** The actual ViewportWorldInteraction */
	UPROPERTY()
	TObjectPtr<UViewportWorldInteraction> ViewportWorldInteraction;

	//
	// Dragging object from UI
	//

	/** The UI used to drag an asset into the level */
	UPROPERTY()
	TObjectPtr<class UWidgetComponent> FloatingUIAssetDraggedFrom;

	/** The material or texture asset we're dragging to place on an object */
	UPROPERTY()
	TObjectPtr<UObject> PlacingMaterialOrTextureAsset;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
