// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Containers/UnrealString.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Paths.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Templates/TypeHash.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define RootToContentDir Style->RootToContentDir

class ISlateStyle;

TSharedPtr< FSlateStyleSet > FVREditorStyle::VREditorStyleInstance = NULL;

void FVREditorStyle::Shutdown()
{
	if (VREditorStyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*VREditorStyleInstance);
		ensure(VREditorStyleInstance.IsUnique());
		VREditorStyleInstance.Reset();
	}
}

FName FVREditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("VREditorStyle"));
	return StyleSetName;
}

FName FVREditorStyle::GetSecondaryStyleSetName()
{
	static FName StyleSetName(TEXT("VRRadialStyle"));
	return StyleSetName;
}

FName FVREditorStyle::GetNumpadStyleSetName()
{
	static FName StyleSetName(TEXT("VRNumpadRadialStyle"));
	return StyleSetName;
}

const FVector2D Icon14x14(14.0f, 14.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon64x64(64.0f, 64.0f);
const FVector2D Icon512x512(512.0f, 512.0f);

TSharedRef< FSlateStyleSet > FVREditorStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet(FVREditorStyle::GetStyleSetName()));

	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Use the default menu button style, but set the background to dark grey.
	const FButtonStyle NormalButton = FAppStyle::GetWidgetStyle<FButtonStyle>("Menu.Button");
	Style->Set("VREditorStyle.Button", FButtonStyle(NormalButton)
		.SetNormal(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.1f, 0.1f, 0.1f))));
	Style->Set("VREditorStyle.CollapsedButton", FButtonStyle(NormalButton)
		.SetNormal(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.1f, 0.1f, 0.1f))));

	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	Style->Set("VREditorStyle.Label", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 7)));

	// Headings will have a font outline
	FFontOutlineSettings HeadingOutline;
	HeadingOutline.OutlineColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.5f);
	HeadingOutline.OutlineSize = 1;
	FSlateFontInfo HeadlineFont = DEFAULT_FONT("Regular", 10);
	HeadlineFont.OutlineSettings = HeadingOutline;

	Style->Set("VREditorStyle.Heading", FTextBlockStyle(NormalText)
			.SetFont(HeadlineFont)
			.SetColorAndOpacity(FLinearColor::White));

	// Headings will have a font outline
	FFontOutlineSettings HelperOutline;
	HelperOutline.OutlineColor = FLinearColor( 0.2f, 0.2f, 0.2f, 0.5f );
	HelperOutline.OutlineSize = 3;
	FSlateFontInfo HelperFont = DEFAULT_FONT( "Regular", 24 );
	HelperFont.OutlineSettings = HelperOutline;

	Style->Set( "VREditorStyle.HelperText", FTextBlockStyle( NormalText )
		.SetFont( HelperFont )
		.SetColorAndOpacity( FLinearColor::White ) );

	FCheckBoxStyle VRMenuCheckBoxStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(IMAGE_BRUSH("Icons/VREditor/T_Radial_Checkbox", Icon64x64, FLinearColor::Transparent))
		.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/VREditor/T_Radial_Checkbox", Icon64x64, FLinearColor::Transparent))
		.SetUncheckedPressedImage(IMAGE_BRUSH("Icons/VREditor/T_Radial_Checkbox", Icon64x64, FLinearColor::Transparent))
		.SetCheckedImage(IMAGE_BRUSH("Icons/VREditor/T_Radial_Checkbox", Icon64x64, FLinearColor::White))
		.SetCheckedHoveredImage(IMAGE_BRUSH("Icons/VREditor/T_Radial_Checkbox", Icon64x64, FLinearColor::White))
		.SetCheckedPressedImage(IMAGE_BRUSH("Icons/VREditor/T_Radial_Checkbox", Icon64x64, FLinearColor::White));

	Style->Set("VREditorStyle.Check", VRMenuCheckBoxStyle);
	Style->Set("VRRadialStyle.Check", VRMenuCheckBoxStyle);

	Style->Set("VREditorStyle.CheckBox", VRMenuCheckBoxStyle);
	Style->Set("VRRadialStyle.CheckBox", VRMenuCheckBoxStyle);

	const FCheckBoxStyle RadioButtonStyle = FAppStyle::GetWidgetStyle<FCheckBoxStyle>("Menu.RadioButton");
	Style->Set("VREditorStyle.RadioButton", FCheckBoxStyle(RadioButtonStyle));
	Style->Set("VRRadialStyle.RadioButton", FCheckBoxStyle(RadioButtonStyle));

	const FCheckBoxStyle ToggleButton = FAppStyle::GetWidgetStyle<FCheckBoxStyle>("Menu.ToggleButton");
	Style->Set("VREditorStyle.ToggleButton", FCheckBoxStyle(ToggleButton));
	Style->Set("VRRadialStyle.ToggleButton", FCheckBoxStyle(ToggleButton));

	const FTextBlockStyle KeybindingStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("Menu.Keybinding");
	Style->Set("VREditorStyle.Keybinding", FTextBlockStyle(KeybindingStyle));
	Style->Set("VRRadialStyle.Keybinding", FTextBlockStyle(KeybindingStyle));

	Style->Set("VREditorStyle.AlignActors", new IMAGE_BRUSH("Icons/UMG/Alignment/Horizontal_Left", Icon16x16));

	Style->Set("VRRadialStyle.Button", FButtonStyle(NormalButton)
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource()));
	Style->Set("VRRadialStyle.CollapsedButton", FButtonStyle(NormalButton)
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource()));

	FFontOutlineSettings RadialOutline;
	RadialOutline.OutlineColor = FLinearColor(0.1f, 0.1f, 0.1f, 1.0f);
	RadialOutline.OutlineSize = 1;

	FSlateFontInfo RadialFont = DEFAULT_FONT("Regular", 8);
	RadialFont.OutlineSettings = RadialOutline;
	Style->Set("VRRadialStyle.Label", FTextBlockStyle(NormalText)
		.SetFont(RadialFont)
		.SetColorAndOpacity(FLinearColor::White));

	Style->Set("VRRadialStyle.InactiveFont", FSlateFontInfo(RadialFont));

	FSlateFontInfo ActiveRadialFont = DEFAULT_FONT("Regular", 10);
	FFontOutlineSettings ActiveRadialOutline;
	ActiveRadialOutline.OutlineColor = FLinearColor::Black;
	ActiveRadialOutline.OutlineSize = 1;
	ActiveRadialFont.OutlineSettings = ActiveRadialOutline;
	Style->Set("VRRadialStyle.ActiveFont", FSlateFontInfo(ActiveRadialFont));
	
	FSlateFontInfo NumpadRadialFont = DEFAULT_FONT("Regular", 24);
	NumpadRadialFont.OutlineSettings = RadialOutline;
	Style->Set("VRNumpadRadialStyle.Label", FTextBlockStyle(NormalText)
		.SetFont(NumpadRadialFont)
		.SetColorAndOpacity(FLinearColor::White));
	Style->Set("VRNumpadRadialStyle.Button", FButtonStyle(NormalButton)
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource()));

	Style->Set("VREditorStyle.EditMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Edit", Icon512x512));
	Style->Set("VREditorStyle.SnapMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Snapping", Icon512x512));
	Style->Set("VREditorStyle.GizmoMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Gizmo", Icon512x512));
	Style->Set("VREditorStyle.ModesMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Modes", Icon512x512));
	Style->Set("VREditorStyle.ToolsMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Tools", Icon512x512));
	Style->Set("VREditorStyle.WindowsMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Windows", Icon512x512));
	Style->Set("VREditorStyle.ActionsMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Actions", Icon512x512));
	
	Style->Set("VREditorStyle.ActorsMode", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Actors", Icon512x512));
	Style->Set("VREditorStyle.FoliageMode", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Foliage", Icon512x512));
	Style->Set("VREditorStyle.LandscapeMode", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Landscape", Icon512x512));
	Style->Set("VREditorStyle.MeshPaintMode", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Mesh_Paint", Icon512x512));

	Style->Set("VREditorStyle.Copy", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Copy", Icon512x512));
	Style->Set("VREditorStyle.Cut", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Cut", Icon512x512));
	Style->Set("VREditorStyle.Paste", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Paste", Icon512x512));
	Style->Set("VREditorStyle.Delete", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Delete", Icon512x512));
	Style->Set("VREditorStyle.Duplicate", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Duplicate", Icon512x512));
	Style->Set("VREditorStyle.SnapToFloor", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Snaps", Icon512x512));
	Style->Set("VREditorStyle.DeselectAll", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Deselect_All", Icon512x512));

	Style->Set("VREditorStyle.Flashlight", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Flashlight", Icon512x512));
	Style->Set("VREditorStyle.Screenshot", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Screenshot", Icon512x512));
	Style->Set("VREditorStyle.Simulate", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Simulate", Icon512x512));
	Style->Set("VREditorStyle.Pause", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Pause", Icon512x512));
	Style->Set("VREditorStyle.Play", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Play", Icon512x512));
	Style->Set("VREditorStyle.Resume", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Resume", Icon512x512));
	Style->Set("VREditorStyle.SaveSimulation", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Save_Actors", Icon512x512));

	Style->Set("VREditorStyle.Translate", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Translate", Icon512x512));
	Style->Set("VREditorStyle.Rotate", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Rotate", Icon512x512));
	Style->Set("VREditorStyle.Scale", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Scale", Icon512x512));
	Style->Set("VREditorStyle.Universal", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Universal", Icon512x512));
	Style->Set("VREditorStyle.WorldSpace", new IMAGE_BRUSH("Icons/VREditor/T_Radial_World_Space", Icon512x512));
	Style->Set("VREditorStyle.LocalSpace", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Local_Space", Icon512x512));

	Style->Set("VREditorStyle.AlignActors", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Align_Actors", Icon512x512));
	Style->Set("VREditorStyle.SetTargets", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Set_Targets", Icon512x512));
	Style->Set("VREditorStyle.GridNum", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Grid_Num", Icon512x512));
	Style->Set("VREditorStyle.TranslateSnap", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Translate_Snap", Icon512x512));
	Style->Set("VREditorStyle.AngleNum", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Angle_Num", Icon512x512));
	Style->Set("VREditorStyle.RotateSnap", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Rotate_Snap", Icon512x512));
	Style->Set("VREditorStyle.ScaleNum", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Scale_Num", Icon512x512));
	Style->Set("VREditorStyle.ScaleSnap", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Scale_Snap", Icon512x512));

	Style->Set("VREditorStyle.ContentBrowser", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Content_Browser", Icon512x512));
	Style->Set("VREditorStyle.Details", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Details", Icon512x512));
	Style->Set("VREditorStyle.ModesPanel", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Modes_Panel", Icon512x512));
	Style->Set("VREditorStyle.Sequencer", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Sequencer", Icon512x512));
	Style->Set("VREditorStyle.WorldOutliner", new IMAGE_BRUSH("Icons/VREditor/T_Radial_World_Outliner", Icon512x512));
	Style->Set("VREditorStyle.WorldSettings", new IMAGE_BRUSH("Icons/VREditor/T_Radial_World_Settings", Icon512x512));

	Style->Set("VREditorStyle.SequencerPlay", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Seq_Play", Icon512x512));
	Style->Set("VREditorStyle.SequencerStop", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Seq_Stop", Icon512x512));
	Style->Set("VREditorStyle.SequencerReverse", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Seq_Reverse", Icon512x512));
	Style->Set("VREditorStyle.Scrub", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Seq_Scrub", Icon512x512));
	Style->Set("VREditorStyle.PlayFromStart", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Seq_Start", Icon512x512));
	Style->Set("VREditorStyle.ToggleLooping", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Seq_Loop", Icon512x512));

	Style->Set("VREditorStyle.Home", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Home_v1", Icon512x512));
	Style->Set("VREditorStyle.OneLevel", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Home_v2", Icon512x512));
	Style->Set("VREditorStyle.TwoLevel", new IMAGE_BRUSH("Icons/VREditor/T_Radial_Home_v3", Icon512x512));

	Style->Set("VREditorStyle.SystemMenu", new IMAGE_BRUSH("Icons/VREditor/T_Radial_VR_Icon", Icon512x512));
	Style->Set("VREditorStyle.ExitVRMode", new IMAGE_BRUSH("Icons/VREditor/T_Radial_VR_Icon", Icon512x512));

	return Style;
}

void FVREditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FVREditorStyle::Get()
{
	if (!VREditorStyleInstance.IsValid())
	{
		VREditorStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*VREditorStyleInstance);
	}
	return *VREditorStyleInstance;
}

#undef RootToContentDir


PRAGMA_ENABLE_DEPRECATION_WARNINGS
