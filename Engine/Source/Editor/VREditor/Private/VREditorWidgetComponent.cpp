// Copyright Epic Games, Inc. All Rights Reserved.

#include "VREditorWidgetComponent.h"
#include "IVREditorModule.h"
#include "VREditorAssetContainer.h"
#include "VREditorMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VREditorWidgetComponent)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


UVREditorWidgetComponent::UVREditorWidgetComponent()
	: Super(),
	DrawingPolicy(EVREditorWidgetDrawingPolicy::Always),
	bIsHovering(false),
	bHasEverDrawn(false)
{
	bSelectable = false;
	SetCastShadow(false);

	if (UNLIKELY(IsRunningDedicatedServer()) || HasAnyFlags(RF_ClassDefaultObject))   // @todo vreditor: early out to avoid loading font assets in the cooker on Linux
	{
		return;
	}

	if (!IsRunningCommandlet())
	{
		if (IVREditorModule::IsAvailable())
		{
			IVREditorModule& VRModule = IVREditorModule::Get();

			if (UVREditorMode* VRMode = VRModule.GetVRMode())
			{
				const UVREditorAssetContainer& VRAssetContainer = VRMode->GetAssetContainer();
				{
					TranslucentMaterial = VRAssetContainer.WidgetMaterial;
					TranslucentMaterial_OneSided = VRAssetContainer.WidgetMaterial;
				}
			}
		}

	}
}


bool UVREditorWidgetComponent::ShouldDrawWidget() const
{
	if ( DrawingPolicy == EVREditorWidgetDrawingPolicy::Always ||
		(DrawingPolicy == EVREditorWidgetDrawingPolicy::Hovering && bIsHovering) ||
		!bHasEverDrawn )
	{
		return Super::ShouldDrawWidget();
	}

	return false;
}

void UVREditorWidgetComponent::DrawWidgetToRenderTarget(float DeltaTime)
{
	bHasEverDrawn = true;

	Super::DrawWidgetToRenderTarget(DeltaTime);
}


PRAGMA_ENABLE_DEPRECATION_WARNINGS
