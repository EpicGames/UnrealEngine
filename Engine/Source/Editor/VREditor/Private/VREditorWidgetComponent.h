// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/WidgetComponent.h"
#include "VREditorWidgetComponent.generated.h"

UENUM()
enum class EVREditorWidgetDrawingPolicy : uint8
{
	Always,
	Hovering,
};

/**
 * A specialized WidgetComponent for the VREditor
 */
UCLASS(hidedropdown)
class UVREditorWidgetComponent : public UWidgetComponent
{
	GENERATED_BODY()

public:
	/** Default constructor that sets up CDO properties */
	UVREditorWidgetComponent();

	void SetDrawingPolicy(EVREditorWidgetDrawingPolicy Value) { DrawingPolicy = Value; }
	EVREditorWidgetDrawingPolicy GetDrawingPolicy() const { return DrawingPolicy; }

	void SetIsHovering(bool Value) { bIsHovering = Value; }
	bool GetIsHovering() const { return bIsHovering; }

protected:
	virtual bool ShouldDrawWidget() const override;

	virtual void DrawWidgetToRenderTarget(float DeltaTime) override;

private:
	/**
	 * High level redrawing policy for the widget component.
	 */
	UPROPERTY()
	EVREditorWidgetDrawingPolicy DrawingPolicy;

	/**
	 * Controls if we draw, the VREditorWidget allows for manual enabling or 
	 * disabling of updating the slate widget.
	 */
	UPROPERTY()
	bool bIsHovering;

	/**
	 * Recorders if we've drawn at least once, that way we can always draw the first
	 * frame then go manual after that.
	 */
	UPROPERTY()
	bool bHasEverDrawn;
};
