// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRModeSettings.h"
#include "Dialogs/Dialogs.h"
#include "UObject/UnrealType.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VRModeSettings)

#define LOCTEXT_NAMESPACE "VREditor"

UVRModeSettings::UVRModeSettings()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	: Super()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
}

#if WITH_EDITOR
void UVRModeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Super::PostEditChangeProperty(PropertyChangedEvent);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (PropertyChangedEvent.Property
		&& PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UVRModeSettings, bEnableAutoVREditMode)
		&& bEnableAutoVREditMode == true)
	{
		FSuppressableWarningDialog::FSetupInfo SetupInfo(LOCTEXT("VRModeEntry_Message", "VR Mode enables you to work on your project in virtual reality using motion controllers. This feature is still under development, so you may experience bugs or crashes while using it."),
			LOCTEXT("VRModeEntry_Title", "Entering VR Mode - Experimental"), "Warning_VRModeEntry", GEditorSettingsIni);

		SetupInfo.ConfirmText = LOCTEXT("VRModeEntry_ConfirmText", "Continue");
		SetupInfo.CancelText = LOCTEXT("VRModeEntry_CancelText", "Cancel");
		SetupInfo.bDefaultToSuppressInTheFuture = true;
		FSuppressableWarningDialog VRModeEntryWarning(SetupInfo);
		bEnableAutoVREditMode = (VRModeEntryWarning.ShowModal() != FSuppressableWarningDialog::Cancel) ? true : false;
	}
}
#endif

#undef LOCTEXT_NAMESPACE
