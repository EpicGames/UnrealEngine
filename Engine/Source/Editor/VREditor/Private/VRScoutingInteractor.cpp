// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRScoutingInteractor.h"

#include "Components/StaticMeshComponent.h"
#include "ViewportWorldInteraction.h"
#include "VREditorMode.h"
#include "VREditorActions.h"
#include "GameFramework/InputSettings.h"
#include "Engine/InputDelegateBinding.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/Selection.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(VRScoutingInteractor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS


#define LOCTEXT_NAMESPACE "UVRScoutingInteractor"

UVRScoutingInteractor::UVRScoutingInteractor() :
	Super(),
	FlyingIndicatorComponent(nullptr)
{
}

float UVRScoutingInteractor::GetSlideDelta_Implementation() const
{
	return 0.0f;
}

void UVRScoutingInteractor::SetupComponent_Implementation(AActor* OwningActor)
{
	Super::SetupComponent_Implementation(OwningActor);

	CreateEditorInput();


	// Flying Mesh
	FlyingIndicatorComponent = NewObject<UStaticMeshComponent>(OwningActor);
	OwningActor->AddOwnedComponent(FlyingIndicatorComponent);
	FlyingIndicatorComponent->SetupAttachment(HandMeshComponent);

	FlyingIndicatorComponent->RegisterComponent();

	FlyingIndicatorComponent->SetMobility(EComponentMobility::Movable);
	FlyingIndicatorComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	FlyingIndicatorComponent->SetCollisionResponseToAllChannels(ECollisionResponse::ECR_Ignore);
	FlyingIndicatorComponent->SetVisibility(false);
	FlyingIndicatorComponent->SetCastShadow(false);

}

void UVRScoutingInteractor::Shutdown_Implementation()
{
	Super::Shutdown_Implementation();

	FlyingIndicatorComponent = nullptr;
	RemoveEditorInput();
}

void UVRScoutingInteractor::SetGizmoMode(EGizmoHandleTypes InGizmoMode)
{
	FVREditorActionCallbacks::SetGizmoMode(&GetVRMode(), InGizmoMode);
}

EGizmoHandleTypes UVRScoutingInteractor::GetGizmoMode() const
{
	return FVREditorActionCallbacks::GetGizmoMode(&GetVRMode());
}

TArray<AActor*> UVRScoutingInteractor::GetSelectedActors()
{
#if WITH_EDITOR
	if (GEditor != nullptr)
	{
		TArray<AActor*> SelectedActors;
		SelectedActors.Reserve(GEditor->GetSelectedActorCount());

		for (auto It = GEditor->GetSelectedActorIterator(); It; ++It)
		{
			if (AActor* Actor = Cast<AActor>(*It))
			{
				SelectedActors.Emplace(Actor);
			}
		}

		return SelectedActors;
	}
#endif
	return TArray<AActor*>();
}

void UVRScoutingInteractor::CreateEditorInput()
{
	EditorOnlyInputComponent = NewObject<UInputComponent>(this, UInputSettings::GetDefaultInputComponentClass(), TEXT("VScoutInputComponent0"), RF_Transient);
	UInputDelegateBinding::BindInputDelegates(GetClass(), EditorOnlyInputComponent, this);
}

void UVRScoutingInteractor::RemoveEditorInput()
{
	ensure(!bReceivesEditorInput);

	if (EditorOnlyInputComponent)
	{
		EditorOnlyInputComponent->DestroyComponent();
	}
	EditorOnlyInputComponent = nullptr;
}

void UVRScoutingInteractor::SetReceivesEditorInput(bool bInValue)
{
	bReceivesEditorInput = bInValue;
	if (bReceivesEditorInput)
	{
		CreateEditorInput();
	}
	else
	{
		RemoveEditorInput();
	}
}

#undef LOCTEXT_NAMESPACE


PRAGMA_ENABLE_DEPRECATION_WARNINGS
