// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SubclassOf.h"

class AActor;

DECLARE_MULTICAST_DELEGATE(FOnVREditingModeEnter);
DECLARE_MULTICAST_DELEGATE(FOnVREditingModeExit);

/**
 * The public interface to this module
 */
class IVREditorModule : public IModuleInterface
{
public:
	/** @return a multicast delegate which is executed when VR mode starts. */
	virtual FOnVREditingModeEnter& OnVREditingModeEnter() = 0;

	/** @return a multicast delegate which is executed when VR mode stops. */
	virtual FOnVREditingModeExit& OnVREditingModeExit() = 0;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IVREditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IVREditorModule >( "VREditor" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "VREditor" );
	}

	/**
	 * Checks whether or not editor VR features are enabled
	 *
	 * @return	True if VR mode is on
	 */
	virtual bool IsVREditorEnabled() const = 0;

	/**
	 * Checks to see whether its possible to use VR mode in this session.  Basically, this makes sure that you
	 * have the appropriate hardware connected
	 *
	 * @return	True if EnableVREditor() can be used to activate VR mode
	 */
	virtual bool IsVREditorAvailable() const = 0;

	/**
	* Checks to see if the VR Mode button should be active or grayed out (such as during SIE)
	*
	* @return	True if button should be active
	*/
	virtual bool IsVREditorButtonActive() const = 0;

	/**
	 * Enables or disables editor VR features.  Calling this to active VR will turn on the HMD and setup
	 * the editor UI for VR interaction.
	 *
	 * @param	bEnable	True to enable VR, or false to turn it off
	 * @param	bForceWithoutHMD	If set to true, will enter VR mode without switching to HMD/stereo.  This can be useful for testing.
	 */
	virtual void EnableVREditor( const bool bEnable, const bool bForceWithoutHMD = false ) = 0;

	/** 
	 * Check if the VR Editor is currently running
	 *
	 * @return True if the VREditor is currently running 
	 */
	UE_DEPRECATED(5.7, "Test whether GetVRModeBase returns nullptr instead; this only returns true for the deprecated UVREditorMode.")
	virtual bool IsVREditorModeActive() = 0;

	/** 
	* Get the current VREditor running
	*
	* @return The current VREditor running
	*/
	UE_DEPRECATED(5.7, "Use GetVRModeBase instead; this only supports the deprecated UVREditorMode.")
	virtual class UVREditorMode* GetVRMode() = 0;

	/** @return The active UVREditorModeBase */
	virtual class UVREditorModeBase* GetVRModeBase() = 0;

	/** 
	* Update the actor preview (for example, the view from a camera attached to a pawn) in VR mode
	*
	* @param The new actor preview widget
	*/
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	virtual void UpdateActorPreview(TSharedRef<class SWidget> InWidget, int32 Index, AActor *Actor = nullptr, bool bIsDetached = false) = 0;


	/**
	* Update any external UMG UI spawned from the radial menu
	*
	* @param The new widget
	* @param The label to use for the UI
	*/
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	virtual void UpdateExternalUMGUI(const struct FVREditorFloatingUICreationContext& CreationContext) = 0;

	/**
	* Update any external Slate UI spawned from the radial menu
	*
	* @param The new widget
	* @param The label to use for the UI
	*/
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	virtual void UpdateExternalSlateUI(TSharedRef<SWidget> InSlateWidget, FName Name, FVector2D InSize = FVector2D::ZeroVector) = 0;

	/** Gets the radial menu extender.  This can be used to add your own menu items to the VR radial menu */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	virtual TSharedPtr<class FExtender> GetRadialMenuExtender() = 0;

};

