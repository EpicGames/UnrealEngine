// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameFramework/Actor.h"
#include "Engine/EngineBaseTypes.h"
#include "VREditorTeleporter.generated.h"

#define UE_API VREDITOR_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UMaterialInstanceDynamic;
class UVREditorInteractor;
class UViewportInteractor;
class UVREditorMode;

/**
 * VR Editor teleport manager and the visual representation of the teleport
 */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") AVREditorTeleporter : public AActor
{
	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UE_API AVREditorTeleporter();

	/** Initializes the teleporter */
	UFUNCTION( BlueprintNativeEvent, CallInEditor, Category = "Teleporter" )
	UE_API void Init( class UVREditorMode* InMode );

	/** Shuts down the teleporter */
	UFUNCTION( BlueprintNativeEvent, CallInEditor, Category = "Teleporter" )
	UE_API void Shutdown();

	/** Whether we are currently aiming to teleport. */
	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API bool IsAiming() const;

	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API bool IsTeleporting() const;

	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API class UVREditorMode* GetVRMode() const;

	/** Start teleporting, does a ray trace with the hand passed and calculates the locations for lerp movement in Teleport */
	UFUNCTION( BlueprintNativeEvent, CallInEditor, Category = "Teleporter" )
	UE_API void StartTeleport();

	/** Called when teleport is done for cleanup */
	UFUNCTION( BlueprintNativeEvent, CallInEditor, Category = "Teleporter" )
	UE_API void TeleportDone();

	/** Hide or show the teleporter visuals */
	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API void SetVisibility( const bool bVisible );

	/** Sets the color for the teleporter visuals */
	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API void SetColor( const FLinearColor& Color );

	/** Get slide delta to push/pull or scale the teleporter */
	UFUNCTION( BlueprintNativeEvent, Category = "Teleporter" )
	UE_API float GetSlideDelta( UVREditorInteractor* Interactor, const bool Axis );

private:

	//~ Begin AActor interface
	UE_API virtual void Tick(const float DeltaTime) override;
	virtual bool IsEditorOnly() const final
	{
		return true;
	}
	//~ End AActor interface

	/** Functions we call to handle teleporting in navigation mode */
	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API void StartAiming( class UViewportInteractor* Interactor );

	/** Cancel teleport aiming mode without doing the teleport */
	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API void StopAiming();

	/** Do and finalize teleport.  	*/
	UFUNCTION( BlueprintCallable, Category = "Teleporter" )
	UE_API void DoTeleport();

	/** Get the actor we're currently trying to teleport with.
	* Valid during aiming and teleporting.
	*/
	UFUNCTION(BlueprintCallable, Category = "Teleporter")
	UE_API UViewportInteractor* GetInteractorTryingTeleport() const;

	/** Called when the user presses a button on their motion controller device */
	UE_API void OnPreviewInputAction(class FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor,
		const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled);

	/** Move the roomspace using lerp towards the new location */
	UE_API void Teleport(const float DeltaTime);

	/** Updating aiming with teleport to end of laser and pulling and pushing */
	UE_API void UpdateTeleportAim(const float DeltaTime);

	/** Helper function to push and pull the teleporter along the laser */
	UE_API FVector UpdatePushPullTeleporter(class UVREditorInteractor* VREditorInteractor, const FVector& LaserPointerStart, const FVector& LaserPointerEnd, const bool bEnablePushPull = true);

	/** Figures out the new transforms for all the visuals based on the new location and the transforms of the HMD and motioncontrollers */
	UE_API void UpdateVisuals(const FVector& NewLocation);

	/** If we want to start showing or hiding the meshes */
	UE_API void Show(const bool bShow);

	UE_API void UpdateFadingState(const float DeltaTime);

	/** Calculated the scale for animation */
	UE_API float CalculateAnimatedScaleFactor() const;

	/** The owning VR mode */
	UPROPERTY()
	TObjectPtr<UVREditorMode> VRMode;

	enum EState
	{
		None = 0,				// When not aiming for a teleport
		Aiming = 1,	// Aiming at the end of the laser, user can also scale using touchpad/analog stick
		Teleporting = 3			// Currently teleporting from one location to another with lerp
	};

	/** The current teleport state */
	EState TeleportingState;

	/** The current lerp of the teleport between the TeleportStartLocation and the TeleportGoalLocation */
	float TeleportLerpAlpha;

	/** Set on the current Roomspace location in the world in StartTeleport before doing the actual teleporting */
	FVector TeleportStartLocation;

	/** The calculated goal location in StartTeleport to move the Roomspace to */
	FVector TeleportGoalLocation;

	/** Visuals for the feet location of the teleporter with the same direction of the HMD yaw */
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> TeleportDirectionMeshComponent;

	/** Visuals for teleport HMD */
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> HMDMeshComponent;

	/** Visuals for teleport left motion controller */
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> LeftMotionControllerMeshComponent;

	/** Visuals for teleport right motion controller */
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> RightMotionControllerMeshComponent;

	/** Dynamic material for teleport visuals */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> TeleportMID;

	/** The interactor that started aiming to teleport */
	UPROPERTY()
	TObjectPtr<UViewportInteractor> InteractorTryingTeleport;

	/** When offset between the hoverlocation of the laser and the calculated  teleport*/
	FVector OffsetDistance;

	/** The goal world to meters scale. This value is used to scale the visuals and will be used to set world to meters scale after teleporting */
	float TeleportGoalScale;

	/** The current length of the laser where the teleport should be at */
	double DragRayLength;

	/** The current drag velocity to push or pull the teleport along the laser */
	double DragRayLengthVelocity;

	/** If the teleporter has been pushed by the trackpad */
	bool bPushedFromEndOfLaser;

	/** Is this the first time aiming for teleporting, we don't have to smooth movement for initial tick */
	bool bInitialTeleportAim;

	/** Fade alpha, for visibility transitions */
	float FadeAlpha;

	/** If the teleporter should fade in */
	TOptional<bool> bShouldBeVisible;

	/** Delay to start the actual moving to the end location */
	uint32 TeleportTickDelay;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API

