// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "VREditorBaseActor.h"
#include "Blueprint/UserWidget.h"
#include "VREditorFloatingUI.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class UVREditorBaseUserWidget;
class UVREditorUISystem;

typedef FName VREditorPanelID;

/**
 * Creation parameters for AVREditorFloatingUI
 */
USTRUCT(BlueprintType)
struct UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") FVREditorFloatingUICreationContext
{
	GENERATED_BODY()

public:	
	
	/** Widget to open in the VR window. null to close an open window (if if matches the PanelID) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	TSubclassOf<UUserWidget> WidgetClass;

	// @todo: As we make this user-definable, we allow possible name collisions - how to deal with that? MakeUniqueName won't work because the name will be different on repeat calls. 
	/** ID that the UI system will use to identify the panel. MUST BE UNIQUE! */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	FName PanelID;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Virtual Production UI")
	TObjectPtr<AActor> ParentActor = nullptr;

	/** Optional offset from HMD where the window opens. Pass FTransform::Identity for default logic - window will open at controller location. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	FTransform PanelSpawnOffset;

	/** Panel size. Should match the size of the UMG passed in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	FVector2D PanelSize = FVector2D(0.0f);

	/** Optional custom mesh to use for the VR window. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	TObjectPtr<UStaticMesh> PanelMesh = nullptr;

	/** Optional override for "VREd.EditorUISize". Leave at 0 for default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	float EditorUISize = 0.0f;

	/** Turn off handles under window? (X-To-Close, movement bar...) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VR Mode UI")
	bool bHideWindowHandles = false;

	/** Turn off the widget's background to create a see-through look. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Virtual Production UI")
	bool bMaskOutWidgetBackground = false;

	/** If bHideWindowHandles is false, this window doesn't have a close button. (*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Virtual Production UI")
	bool bNoCloseButton = false;
};


/**
 * Represents an interactive floating UI panel in the VR Editor
 */
UCLASS()
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") AVREditorFloatingUI : public AVREditorBaseActor
{
	GENERATED_BODY()

public:

	/** Default constructor which sets up safe defaults */
	AVREditorFloatingUI(const FObjectInitializer& ObjectInitializer);

	/** Creates a FVREditorFloatingUI using a Slate widget, and sets up safe defaults */
	void SetSlateWidget( class UVREditorUISystem& InitOwner, const VREditorPanelID& InID, const TSharedRef<SWidget>& InitSlateWidget, const FIntPoint InitResolution, const float InitScale, const EDockedTo InitDockedTo );
	void SetSlateWidget(const TSharedRef<SWidget>& InitSlateWidget);

	/** Creates a FVREditorFloatingUI using a UMG user widget, and sets up safe defaults */
	void SetUMGWidget(class UVREditorUISystem& InitOwner, const VREditorPanelID& InID, class TSubclassOf<class UUserWidget> InitUserWidgetClass, const FIntPoint InitResolution, const float InitScale, const EDockedTo InitDockedTo);

	virtual void TickManually(float DeltaTime) override;

	/** @return Returns true if the UI is visible (or wants to be visible -- it might be transitioning */
	bool IsUIVisible() const
	{
		return bShouldBeVisible.GetValue();
	}

	/** Shows or hides the UI (also enables collision, and performs a transition effect) */
	void VREDITOR_API ShowUI( const bool bShow, const bool bAllowFading = true, const float InitFadeDelay = 0.0f, const bool bInClearWidgetOnHide = false );

	/** Sets the resolution of this floating UI panel and resets the window mesh accordingly. */
	void SetResolution(const FIntPoint& InResolution);

	/** Returns the widget component for this UI, or nullptr if not spawned right now */
	class UVREditorWidgetComponent* GetWidgetComponent()
	{
		return WidgetComponent;
	}

	/** Returns the mesh component for this UI, or nullptr if not spawned right now */
	class UStaticMeshComponent* GetMeshComponent()
	{
		return WindowMeshComponent;
	}

	/** Returns the actual size of the UI in either axis, after scaling has been applied.  Does not take into animation or world scaling */
	FVector2D GetSize() const;

	/** Gets the scale */
	float GetScale() const;

	/** Sets a new size for the UI */
	void SetScale(const float NewSize, const bool bScaleWidget = true);

	/** Set the widget scale */
	void SetWidgetComponentScale(const FVector& InScale);

	/** Sets the UI transform */
	virtual void SetTransform( const FTransform& Transform ) override;

	/** AActor overrides */
	virtual void BeginDestroy() override;
	virtual void Destroyed() override;

	void CleanupWidgetReferences();

	virtual bool IsEditorOnly() const final
	{
		return true;
	}
	//~ End AActor interface

	/** Returns the owner of this object */
	UVREditorUISystem& GetOwner()
	{
		check( Owner != nullptr );
		return *Owner;
	}

	/** Returns the owner of this object (const) */
	const UVREditorUISystem& GetOwner() const
	{
		check( Owner != nullptr );
		return *Owner;
	}

	/** Returns casted userwidget */
	template<class T>
	T* GetUserWidget() const
	{
		return CastChecked<T>( UserWidget );
	}

	/** Gets the current user widget of this floating UI, return nullptr if using slate widget */
	VREDITOR_API UUserWidget* GetUserWidget();

	/** Gets the initial size of this UI */
	float GetInitialScale() const;

	/** Called to finish setting everything up, after a widget has been assigned */
	virtual void SetupWidgetComponent();

	/** Gets the ID of this panel. */
	VREditorPanelID GetID() const;

	/** Gets the current slate widget. */
	TSharedPtr<SWidget> GetSlateWidget() const;

	/** Set mesh on window mesh component. */
	void SetWindowMesh(class UStaticMesh* InWindowMesh);

	/** All params used to create this panel if this panel has a UMG widget and was created via BP. Invalid otherwise. */
	UPROPERTY()
	FVREditorFloatingUICreationContext CreationContext;	

protected:

	/** Returns a scale to use for this widget that takes into account animation */
	FVector CalculateAnimatedScale() const;

	/** Set collision on components */
	virtual void SetCollision(const ECollisionEnabled::Type InCollisionType, const ECollisionResponse InCollisionResponse, const ECollisionChannel InCollisionChannel);

private:

	/** Called after spawning, and every Tick, to update opacity of the widget */
	virtual void UpdateFadingState( const float DeltaTime );

protected:
	/** Slate widget we're drawing, or null if we're drawing a UMG user widget */
	TSharedPtr<SWidget> SlateWidget;
	
	/** UMG user widget we're drawing, or nullptr if we're drawing a Slate widget */
	UPROPERTY()
	TObjectPtr<UUserWidget> UserWidget;
	
	/** When in a spawned state, this is the widget component to represent the widget */
	UPROPERTY()
	TObjectPtr<class UVREditorWidgetComponent> WidgetComponent;

	/** The floating window mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> WindowMeshComponent;

	/** Resolution we should draw this UI at, regardless of scale */
	FIntPoint Resolution;

	float WorldPlacedScaleFactor;

private:

	/** Owning object */
	class UVREditorUISystem* Owner;

	/** Class of the UMG widget we're spawning */
	UPROPERTY()
	TObjectPtr<UClass> UserWidgetClass;

	/** True if the UI wants to be visible, or false if it wants to be hidden.  Remember, we might still be visually 
	    transitioning between states */
	TOptional<bool> bShouldBeVisible;

	/** Fade alpha, for visibility transitions */
	float FadeAlpha;

	/** Delay to fading in or out. Set on ShowUI and cleared on finish of fade in/out */
	float FadeDelay;

	/** The starting scale of this UI */
	float InitialScale;

	/** The ID of this floating UI. */
	VREditorPanelID UISystemID;

	/** Null out the widget when hidden. */
	bool bClearWidgetOnHide;

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
