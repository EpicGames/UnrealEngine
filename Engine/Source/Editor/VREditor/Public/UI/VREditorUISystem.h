// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Widgets/SOverlay.h"
#include "UI/VRRadialMenuHandler.h"
#include "EditorWorldExtension.h"
#include "VREditorUISystem.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class AVREditorDockableWindow;
class AVREditorFloatingUI;
class FProxyTabmanager;
struct FTabId;
class SColorPicker;
class SBorder;
class SButton;
class FMultiBox;
class SMultiBoxWidget;
class SWidget;
class UViewportInteractor;
class UVREditorInteractor;
class FMenuBuilder;
class FUICommandList;
class UVREditorWidgetComponent;
class UWidgetComponent;
class FName;
typedef FName FEditorModeID;

typedef FName VREditorPanelID;

enum class EAssetEditorCloseReason : uint8;

/** Stores the animation playback state of a VR UI element */
enum class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") EVREditorAnimationState : uint8
{
	None,
	Forward,
	Backward
};

/** Structure to keep track of all relevant interaction and animation elements of a VR Button */
USTRUCT()
struct UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") FVRButton
{
	GENERATED_USTRUCT_BODY()

	/** Pointer to button */
	UPROPERTY()
	TObjectPtr<UVREditorWidgetComponent> ButtonWidget;

	/** Animation playback state of the button */
	EVREditorAnimationState AnimationDirection;

	/** Original relative scale of the button element */
	FVector OriginalRelativeScale;

	/** Current scale of the button element */
	float CurrentScale;

	/** Minimum scale of the button element */
	float MinScale;

	/** Maximum scale of the button element */
	float MaxScale;

	/** Rate at which the button changes scale. Currently the same for scaling up and scaling down. */
	float ScaleRate;

	FVRButton()
		: ButtonWidget(nullptr),
		AnimationDirection(EVREditorAnimationState::None),
		OriginalRelativeScale(FVector::ZeroVector),
		CurrentScale(1.0f),
		MinScale(1.0f),
		MaxScale(1.10f),
		ScaleRate(2.0f)
		{}

	FVRButton(class UVREditorWidgetComponent* InButtonWidget, FVector InOriginalScale,
		EVREditorAnimationState InAnimationDirection = EVREditorAnimationState::None, float InCurrentScale = 1.0f, float InMinScale = 1.0f, float InMaxScale = 1.25f, float InScaleRate = 2.0f)
		: ButtonWidget(InButtonWidget),
		AnimationDirection(InAnimationDirection),
		OriginalRelativeScale(InOriginalScale),
		CurrentScale(InCurrentScale),
		MinScale(InMinScale),
		MaxScale(InMaxScale),
		ScaleRate(InScaleRate)
		{}

};


/**
 * VR Editor user interface manager
 */
UCLASS()
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") UVREditorUISystem : public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UVREditorUISystem();

	/** Initializes default values for the UISystem and creates the UI panels */
	void Init(class UVREditorMode* InVRMode);

	void UpdateInteractors();

	/** Shuts down the UISystem whenever the mode is exited */
	void Shutdown();

	/** Called by VREditorMode to update us every frame */
	void Tick( class FEditorViewportClient* ViewportClient, const float DeltaTime );

	/** Called by VREditorMode to draw debug graphics every frame */
	void Render( const class FSceneView* SceneView, class FViewport* Viewport, class FPrimitiveDrawInterface* PDI );

	/** Gets the owner of this system */
	class UVREditorMode& GetOwner()
	{
		return *VRMode;
	}

	/** Gets the owner of this system (const) */
	const class UVREditorMode& GetOwner() const
	{
		return *VRMode;
	}

	/** Returns true if the specified editor UI panel is currently visible */
	bool VREDITOR_API IsShowingEditorUIPanel(const VREditorPanelID& InPanelID) const;

	/** Sets whether the specified editor UI panel should be visible.  Any other UI floating off this hand will be dismissed when showing it. */
	void ShowEditorUIPanel(const UWidgetComponent* WidgetComponent, UVREditorInteractor* Interactor, const bool bShouldShow, const bool bSpawnInFront = false, const bool bDragFromOpen = false, const bool bPlaySound = true);
	void ShowEditorUIPanel(const VREditorPanelID& InPanelID, UVREditorInteractor* Interactor, const bool bShouldShow, const bool bSpawnInFront = false, const bool bDragFromOpen = false, const bool bPlaySound = true);
	void ShowEditorUIPanel(AVREditorFloatingUI* Panel, UVREditorInteractor* Interactor, const bool bShouldShow, const bool bSpawnInFront = false, const bool bDragFromOpen = false, const bool bPlaySound = true);

	/** Returns true if the radial menu is visible on this hand */
	bool IsShowingRadialMenu(const UVREditorInteractor* Interactor ) const;

	/** Tries to spawn the radial menu (if the specified hand isn't doing anything else) */
	void TryToSpawnRadialMenu( UVREditorInteractor* Interactor, const bool bForceRefresh, const bool bPlaySound = true );

	/** Hides the radial menu if the specified hand is showing it */
	void HideRadialMenu(const bool bPlaySound = true, const bool bAllowFading = true);

	/** Start dragging a dock window on the hand */
	void StartDraggingDockUI( class AVREditorDockableWindow* InitDraggingDockUI, UVREditorInteractor* Interactor, const float DockSelectDistance, const bool bPlaySound = true );

	/** Makes up a transform for a dockable UI when dragging it with a laser at the specified distance from the laser origin */
	FTransform MakeDockableUITransformOnLaser( AVREditorDockableWindow* InitDraggingDockUI, UVREditorInteractor* Interactor, const float DockSelectDistance ) const;

	/** Makes up a transform for a dockable UI when dragging it that includes the original offset from the laser's impact point */
	FTransform MakeDockableUITransform( AVREditorDockableWindow* InitDraggingDockUI, UVREditorInteractor* Interactor, const float DockSelectDistance );
	
	/** Returns the current dragged dock window, nullptr if none */
	class AVREditorDockableWindow* GetDraggingDockUI() const;

	/** Resets all values to stop dragging the current dock window */
	void StopDraggingDockUI( UVREditorInteractor* VREditorInteractor, const bool bPlaySound = true );

	/** Are we currently dragging a dock window */
	bool IsDraggingDockUI();

	bool IsInteractorDraggingDockUI( const UVREditorInteractor* Interactor ) const;

	/** If a panel was opened and dragged with the UI interactor */
	bool IsDraggingPanelFromOpen() const;

	/** Hides and unhides all the editor UI panels */
	void TogglePanelsVisibility();

	/** Get the maximum dock window size */
	float GetMaxDockWindowSize() const;

	/** Get the minimum dock window size */
	float GetMinDockWindowSize() const;

	/** Toggles the visibility of the panel, if the panel is in room space it will be hidden and docked to nothing */
	void TogglePanelVisibility(const VREditorPanelID& InPanelID);

	/** Returns the radial widget so other classes, like the interactors, can access its functionality */
	class AVREditorRadialFloatingUI* GetRadialMenuFloatingUI() const;

	/** 
	 * Finds a widget with a given name inside the Content argument 
	 * @param Content The widget to begin searching in
	 * @param The FName of the widget type to find. 
	 */
	static const TSharedRef<SWidget>& FindWidgetOfType(const TSharedRef<SWidget>& Content, FName WidgetType);

	/**
	* Finds a widget with a given name inside the Content argument
	* @param Content The widget to begin searching in
	* @param The FName of the widget type to find.
	*/
	static const bool FindAllWidgetsOfType(TArray<TSharedRef<SWidget>>& FoundWidgets, const TSharedRef<SWidget>& Content, FName WidgetType);

	/** Function to force an update of the Sequencer UI based on a change */
	void UpdateSequencerUI();

	/** Function to force an update of the Actor Preview UI based on a change */
	void UpdateActorPreviewUI(TSharedRef<SWidget> InWidget, int32 Index, AActor *InActor );

	/** Function to force an update of the Actor Preview UI based on a change when the preview is detached from the actor */
	void UpdateDetachedActorPreviewUI(TSharedRef<SWidget> InWidget, int32 Index);
	
	void UpdateExternalUMGUI(const struct FVREditorFloatingUICreationContext& CreationContext);

	void UpdateExternalSlateUI(TSharedRef<SWidget> InWidget, FName Name, FVector2D InSize);

	/** Transition the user widgets to a new world */
	void TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState);

	UVRRadialMenuHandler* GetRadialMenuHandler()
	{
		return RadialMenuHandler;
	}

	/** Called when a laser or simulated mouse hover enters a button */
	void OnHoverBeginEffect(UVREditorWidgetComponent* Button);
	/** Called when a laser or simulated mouse hover leaves a button */
	void OnHoverEndEffect(UVREditorWidgetComponent* Button);

	/** Set if sequencer was opened from the radial menu */
	void SequencerOpenendFromRadialMenu(const bool bInOpenedFromRadialMenu = true);

	/** If a dockable window can be scaled */
	bool CanScalePanel() const;

	/** Get the interactor that holds the radial menu */
	class UVREditorInteractor* GetUIInteractor();

	static const VREditorPanelID ContentBrowserPanelID;
	static const VREditorPanelID WorldOutlinerPanelID;
	static const VREditorPanelID DetailsPanelID;
	static const VREditorPanelID ModesPanelID;
	static const VREditorPanelID TutorialPanelID;
	static const VREditorPanelID WorldSettingsPanelID;
	static const VREditorPanelID ColorPickerPanelID;
	static const VREditorPanelID SequencerPanelID;
	static VREDITOR_API const VREditorPanelID InfoDisplayPanelID;
	static const VREditorPanelID RadialMenuPanelID;
	static const VREditorPanelID TabManagerPanelID;

	/** Get UI panel Actor from the passed ID */
	VREDITOR_API AVREditorFloatingUI* GetPanel(const VREditorPanelID& InPanelID) const;

protected:

	/** Called to "preview" an input event to get a first chance at it. */
	void OnPreviewInputAction( class FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor,
		const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled );

	/** Called when the user presses a button on their motion controller device */
	void OnVRAction( class FEditorViewportClient& ViewportClient, UViewportInteractor* Interactor,
		const struct FViewportActionKeyInput& Action, bool& bOutIsInputCaptured, bool& bWasHandled );

	/** Called every frame to update hover state */
	void OnVRHoverUpdate( UViewportInteractor* Interactor, FVector& HoverImpactPoint, bool& bWasHandled );

	/** Testing Slate UI */
	void CreateUIs();

	/** Called by proxy tab manager to ask us whether we support the specified tab ID.  bOutIsTabSupported defaults to true. */
	void IsProxyTabSupported( FTabId TabId, bool& bOutIsTabSupported );

	/**
	 * Called when the injected proxy tab manager reports a new tab has been launched, 
	 * signaling an asset editor has been launched, but really it could be any major global tab.
	 */
	void OnProxyTabLaunched(TSharedPtr<SDockTab> NewTab);

	/** Called when the injected proxy tab manager reports needing to draw attention to a tab. */
	void OnAttentionDrawnToTab(TSharedPtr<SDockTab> NewTab);

	/** Show the asset editor if it's not currently visible. */
	void ShowAssetEditor();

	/** Called when an asset editor opens an asset while in VR Editor Mode. */
	void OnAssetEditorOpened(UObject* Asset);

	/** Can be used when the tab manager is restoring a dockable tab area */
	void DockableAreaRestored();

	/** Creates a VR-specific color picker. Gets bound to SColorPicker's creation override delegate */
	void CreateVRColorPicker(const TSharedRef<SColorPicker>& ColorPicker);

	/** Hides the VR-specific color picker. Gets bound to SColorPicker's destruction override delegate */
	void DestroyVRColorPicker();

	/**  Makes a uniform grid widget from the menu information contained in a MultiBox and MultiBoxWidget */
	void MakeUniformGridMenu( const TSharedRef<FMultiBox>& MultiBox, const TSharedRef<SMultiBoxWidget>& MultiBoxWidget, int32 Columns);
	/**  Makes a radial box widget from the menu information contained in a MultiBox and MultiBoxWidget */
	void MakeRadialBoxMenu(const TSharedRef<FMultiBox>& MultiBox, const TSharedRef<SMultiBoxWidget>& MultiBoxWidget, float RadiusRatioOverride, FName ButtonTypeOverride);

	/** Adds a hoverable button of a given type to an overlay, using menu data from a BlockWidget */
	TSharedRef<SWidget> AddHoverableButton(TSharedRef<SWidget>& BlockWidget, FName ButtonType, TSharedRef<SOverlay>& TestOverlay);
	/** Sets the text wrap size of the text block element nested in a BlockWidget */
	TSharedRef<SWidget> SetButtonFormatting(TSharedRef<SWidget>& BlockWidget, float WrapSize);
	
	/** Builds the radial menu Slate widget */
	void BuildRadialMenuWidget();
	/** Builds the numpad Slate widget */
	void BuildNumPadWidget();
	/** Swaps the content of the radial menu between the radial menu and the numpad */
	void SwapRadialMenu();
	/**
	* Handles being notified when any editor mode changes to see if any VR Editor UI needs to change.
	* @param Mode The mode that changed.
	* @param IsEnabled true if the mode is enabled, otherwise false.
	*/
	void HandleEditorModeChanged(const FEditorModeID& EditorModeID, bool IsEnabled);

	/** Reset function that puts you back in placement mode, closes all UIs, etc. */
	void ResetAll();

	/**The VR Editor UI System's rules for when drag drop should be checked for */
	bool CheckForVRDragDrop();

	/** Preview the UI panel's location if spawning with the UI interactor, else spawn immediately */
	bool ShouldPreviewPanel();

	/** When VR Mode debug was toggled. */
	void ToggledDebugMode(bool bDebugModeEnabled);

	void OnCloseAssetEditor(UObject* Asset, EAssetEditorCloseReason CloseReason);
protected:

	/** Owning object */
	UPROPERTY()
	TObjectPtr<class UVREditorMode> VRMode;

	/** All of the floating UIs.  These may or may not be visible (spawned) */
	UPROPERTY()
	TMap<FName, TObjectPtr<class AVREditorFloatingUI>> FloatingUIs;

	/** All of the preview window info. */
	UPROPERTY()
	TMap<FName, TObjectPtr<class AActor>> PreviewWindowInfo;

	/** Our Quick Menu UI */
	UPROPERTY()
	TObjectPtr<class AVREditorFloatingUI> InfoDisplayPanel;

	/** 
	 * The current widget used on the info display. Often we wrap a widget in a widget to configure the settings (e.g. DPI). 
	 * To check the info display widget with other widgets we need that wrapper widget.
	 */
	TWeakPtr<SWidget> CurrentWidgetOnInfoDisplay;

	/** The Radial Menu UI */
	UPROPERTY()
	TObjectPtr<class AVREditorRadialFloatingUI> QuickRadialMenu;

	/** The time since the radial menu was updated */
	float RadialMenuHideDelayTime;

	/** True if the radial menu was visible when the content was swapped */
	bool bRadialMenuVisibleAtSwap;

	/** True if the radial menu is currently displaying the numpad */
	bool bRadialMenuIsNumpad;
	
	//
	// Dragging UI
	//

	/** Interactor that is dragging the UI */
	class UVREditorInteractor* InteractorDraggingUI;

	/** Offset transform from room-relative transform to the object where we picked it up at */
	FTransform DraggingUIOffsetTransform;

	/** The current UI that is being dragged */
	UPROPERTY()
	TObjectPtr<class AVREditorDockableWindow> DraggingUI;

	/** The color picker dockable window */
	UPROPERTY()
	TObjectPtr<class AVREditorDockableWindow> ColorPickerUI;

	//
	// Asymmetry
	//
	/** Interactor that has a laser and is generally interacting with the scene */
	UPROPERTY()
	TObjectPtr<class UVREditorInteractor> LaserInteractor;
	/** Interactor that usually accesses UI and other helper functionality */
	UPROPERTY()
	TObjectPtr<class UVREditorInteractor> UIInteractor;

	//
	// Tab Manager UI
	//

	/** Allows us to steal the global tabs and show them in the world. */
	TSharedPtr<FProxyTabmanager> ProxyTabManager;

	/** Set to true when we need to refocus the viewport. */
	bool bRefocusViewport;

	/** The last dragged hover location by the laser */
	FVector LastDraggingHoverLocation;

	/** All buttons created for the radial and quick menus */
	UPROPERTY()
	TArray<FVRButton> VRButtons;

	/** The add-on that handles radial menu switching */
	UPROPERTY()
	TObjectPtr<UVRRadialMenuHandler> RadialMenuHandler;

	/** When replacing the actions menu, store off any existing actions */
	FOnRadialMenuGenerated ExistingActionsMenu;

	/** When replacing the actions menu, store off the name of the existing actions menu */
	FText ExistingActionsMenuLabel;

	/** The time the modifier was pressed at to spawn the menu */
	FTimespan RadialMenuModifierSpawnTime;

	/** If sequenced was opened from the radial menu or somewhere else such as the content browser. */
	bool bSequencerOpenedFromRadialMenu;

	/** If started dragging from opening a UI panel */
	bool bDragPanelFromOpen;

	/** The time dragging a panel that was opened resulting in an instant drag */
	float DragPanelFromOpenTime;

	/** When started dragging from the radial menu we want the analog stick to be reset before the user is allowed to scale the panel. 
	    Otherwise the panel will immediately start scaling because the user is using the analog stick to aim at the radial menu items. */
	bool bPanelCanScale;

	FDelegateHandle AssetEditorCloseDelegate;
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
