// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "VRRadialMenuHandler.generated.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS


class FUICommandList;
class UVREditorMode;

UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
DECLARE_DELEGATE_FourParams(FOnRadialMenuGenerated, FMenuBuilder&, TSharedPtr<FUICommandList>, UVREditorMode*, float&);

/**
* VR Editor user interface manager
*/
UCLASS()
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") UVRRadialMenuHandler : public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UVRRadialMenuHandler();

	/** Builds the current radial menu */
	void BuildRadialMenuCommands(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride);

	/** Registers a new menu generator and replaces the currently displayed menu if the radial menu is open */
	void RegisterMenuGenerator(const FOnRadialMenuGenerated NewMenuGenerator, const bool bShouldAddToStack = true);

	FOnRadialMenuGenerated GetCurrentMenuGenerator()
	{
		return OnRadialMenuGenerated;
	};

	FOnRadialMenuGenerated GetHomeMenuGenerator()
	{
		return HomeMenu;
	};

	/** Returns to the previous radial menu */
	void BackOutMenu();

	/** Returns to the home menu */
	void Home();

	void Init(class UVREditorUISystem* InUISystem);

protected:
	/** Functions to bind to each menu delegate */
	void HomeMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride);

	void SnapMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride);

	void GizmoMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride);

	void EditMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride);

	void SystemMenuGenerator(FMenuBuilder& MenuBuilder, TSharedPtr<FUICommandList> CommandList, UVREditorMode* VRMode, float& RadiusOverride);

protected:

	FOnRadialMenuGenerated OnRadialMenuGenerated;

	FOnRadialMenuGenerated HomeMenu;

	FOnRadialMenuGenerated SnapMenu;

	FOnRadialMenuGenerated GizmoMenu;

	FOnRadialMenuGenerated EditMenu;

	FOnRadialMenuGenerated SystemMenu;

	static FText ActionMenuLabel;

	class UVREditorUISystem* UIOwner;

	TArray<FOnRadialMenuGenerated> MenuStack;

};


PRAGMA_ENABLE_DEPRECATION_WARNINGS
