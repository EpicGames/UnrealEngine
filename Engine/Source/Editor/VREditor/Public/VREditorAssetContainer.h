// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "VREditorAssetContainer.generated.h"


// Forward declarations
class USoundBase;
class USoundCue;
class UStaticMesh;
class UMaterial;
class UMaterialInterface;
class UMaterialInstance;
class UFont;

/**
 * Asset container for VREditor.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") UVREditorAssetContainer : public UDataAsset
{
	GENERATED_BODY()

public:
	
	//
	// Sounds
	//

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> DockableWindowCloseSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> DockableWindowOpenSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> DockableWindowDropSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> DockableWindowDragSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> DropFromContentBrowserSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> RadialMenuOpenSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> RadialMenuCloseSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> TeleportSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundCue> ButtonPressSound;

	UPROPERTY(EditAnywhere, Category = Sound)
	TObjectPtr<USoundBase> AutoScaleSound;

	//
	// Meshes
	//

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> GenericHMDMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> PlaneMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> CylinderMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerStartMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerEndMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerHoverMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> VivePreControllerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> OculusControllerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> GenericControllerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> TeleportRootMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> WindowMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> WindowSelectionBarMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> WindowCloseButtonMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> RadialMenuMainMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> RadialMenuPointerMesh;
	
	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> PointerCursorMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LineSegmentCylinderMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> JointSphereMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> DockingButtonMesh;

	//
	// Materials
	//

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> GridMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> LaserPointerMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> LaserPointerTranslucentMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterial> WorldMovementPostProcessMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> TextMaterial;
	
	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> VivePreControllerMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> OculusControllerMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> TeleportMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> WindowMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> WindowTranslucentMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterial> LineMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> TranslucentTextMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> WidgetMaterial;

	//Specific material for camera widgets that operates in linear color space
	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> CameraWidgetMaterial;


	//
	// Fonts
	//

	UPROPERTY(EditAnywhere, Category = Font)
	TObjectPtr<UFont> TextFont;
};
