// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "VREditorBaseActor.generated.h"

/**
* Represents an actor specifically for VR Editor that has roomspace transformation
*/
UCLASS()
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") AVREditorBaseActor : public AActor
{
	GENERATED_BODY()

public:

	/** Default constructor which sets up safe defaults */
	AVREditorBaseActor();

	/** Sets the owning VR mode */
	void SetVRMode( class UVREditorMode* InVRMode );

	/** Possible UI attachment points */
	enum class EDockedTo
	{
		Nothing,
		LeftHand,
		RightHand,
		LeftArm,
		RightArm,
		Room,
		Custom,
		Dragging,
	};

	/** Called by the UI system to tick this UI every frame.  We can't use AActor::Tick() because this actor can exist in 
		the editor's world, which never ticks. */
	virtual void TickManually( float DeltaTime );

	/** Sets the transf */
	virtual void SetTransform( const FTransform& Transform ) {};
	
	/** Sets the relative offset of the UI */
	void SetRelativeOffset( const FVector& InRelativeOffset );

	/** Sets the local rotation of the UI */
	void SetLocalRotation( const FRotator& InRelativeRotation );

	/** Returns what we're docked to, if anything */
	EDockedTo GetDockedTo() const
	{
		return DockedTo;
	}

	/** Returns what we were previously docked to */
	EDockedTo GetPreviouslyDockedTo() const
	{
		return PreviousDockedTo;
	}

	/** Sets what this UI is docked to */
	void SetDockedTo( const EDockedTo NewDockedTo );

	/** Start moving towards a transform */
	void MoveTo( const FTransform& ResultTransform, const float TotalMoveToTime, const EDockedTo ResultDock );

	/** Abort moving to transform and move instant to MoveToTransform */
	void StopMoveTo();

	/** Get a relative transform from the hand */
	FTransform MakeUITransformLockedToHand( class UViewportInteractor* Interactor, const bool bOnArm, const FVector& InRelativeOffset, const FRotator& InLocalRotation );
	
protected:

	/** Called every tick to keep the UI position up to date */
	void UpdateTransformIfDocked();

	/** Given a hand to lock to, returns a transform to place UI at that hand's location and orientation */
	FTransform MakeUITransformLockedToHand( class UViewportInteractor* Interactor, const bool bOnArm );
	

	/** Create a room transform with the RelativeOffset and LocalRotation */
	FTransform MakeUITransformLockedToRoom();

	/** Updates the lerp movement */
	void TickMoveTo( const float DeltaTime );

	/** Called when DockTo state is custom, needs to be implemented by derived classes */
	virtual FTransform MakeCustomUITransform() { return FTransform(); };

	/** Called after spawning, and every Tick, to update opacity of the actor, has to be implemented by the derived actor */
	virtual void UpdateFadingState( const float DeltaTime ) {};

	/** How big the actor should be */
	float Scale;

protected:

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** The VR mode that owns this actor */
	UPROPERTY()
	TObjectPtr<class UVREditorMode> VRMode;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	/** Local rotation of the UI */
	FRotator LocalRotation;

	/** Relative offset of the UI */
	FVector RelativeOffset;

	/** What the UI is attached to */
	EDockedTo DockedTo;

	/** What was the UI previously attached to */
	EDockedTo PreviousDockedTo;

	/** If the UI is currently moving */
	bool bIsMoving;

	/** The end transform to move to */
	FTransform MoveToTransform;

	/** The actor transform when started moving */
	FTransform StartMoveToTransform;

	/** Current lerp alpha */
	float MoveToAlpha;

	/** Total time to move to the end transform */
	float MoveToTime;

	/** Dock state to switch to when finished moving */
	EDockedTo MoveToResultDock;

	/** When docked, our transform last frame.  This is used for smoothing */
	TOptional<FTransform> LastDockedUIToWorld;
};
