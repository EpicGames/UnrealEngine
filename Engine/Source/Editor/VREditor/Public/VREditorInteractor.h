// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "InputCoreTypes.h"
#include "ViewportInteractor.h"
#include "UObject/ObjectMacros.h"
#include "VREditorInteractor.generated.h"

#define UE_API VREDITOR_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class AActor;
class UWidgetComponent;
class UStaticMeshSocket;
class AVREditorTeleporter;

UENUM( BlueprintType )
enum class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") EControllerType : uint8
{
	Laser,
	AssistingLaser,
	UI,
	Navigation,
	Unknown
};

/** Directions the trackpad can be swiped to */
UENUM( BlueprintType )
enum class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") ETouchSwipeDirection : uint8
{
	None = 0,
	Left = 1,
	Right = 2,
	Up = 3,
	Down = 4
};

/**
 * VREditor default interactor
 */
UCLASS(MinimalAPI,  Blueprintable, BlueprintType )
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") UVREditorInteractor : public UViewportInteractor
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UE_API UVREditorInteractor();

	/** Gets the owner of this system */
	class UVREditorMode& GetVRMode()
	{
		return *VRMode;
	}

	/** Gets the owner of this system (const) */
	const class UVREditorMode& GetVRMode() const
	{
		return *VRMode;
	}

	/** Gets the key-to-action map (const) */
	const TMap<FKey, FViewportActionKeyInput>& GetKeyToActionMap() const
	{
		return KeyToActionMap;
	}

	UE_API virtual TMap<FViewportActionKeyInput, TArray<FKey>> GetKnownActionMappings(EControllerHand InHand = EControllerHand::AnyHand, FName InHMDDeviceType = NAME_None) const;

	/** Initialize default values */
	UFUNCTION( BlueprintNativeEvent, CallInEditor, Category = "UVREditorInteractor" )
	UE_API void Init( class UVREditorMode* InVRMode );

	/** Sets up all components */
	UFUNCTION( BlueprintNativeEvent, CallInEditor, Category = "UVREditorInteractor" )
	UE_API void SetupComponent( AActor* OwningActor );


	// ViewportInteractorInterface overrides
	UE_API virtual void Shutdown_Implementation() override;
	UE_API virtual void Tick_Implementation( const float DeltaTime ) override;
	UE_API virtual void CalculateDragRay( double& InOutDragRayLength, double& InOutDragRayVelocity ) override;

	/** @return Returns the type of HMD we're dealing with */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API FName GetHMDDeviceType() const;

	UE_API virtual FHitResult GetHitResultFromLaserPointer( TArray<AActor*>* OptionalListOfIgnoredActors = nullptr, const EHitResultGizmoFilterMode GizmoFilerMode = EHitResultGizmoFilterMode::All,
	TArray<UClass*>* ObjectsInFrontOfGizmo = nullptr, const bool bEvenIfBlocked = false, const float LaserLengthOverride = 0.0f ) override;
	UE_API virtual void ResetHoverState() override;
	UE_API virtual bool IsModifierPressed() const override;
	UE_API virtual void PreviewInputKey( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled ) override;
	UE_API virtual void HandleInputKey( class FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const EInputEvent Event, bool& bOutWasHandled ) override;
	UE_API virtual bool GetTransformAndForwardVector( FTransform& OutHandTransform, FVector& OutForwardVector ) const override;
	UE_API virtual void HandleInputAxis( FEditorViewportClient& ViewportClient, FViewportActionKeyInput& Action, const FKey Key, const float Delta, const float DeltaTime, bool& bOutWasHandled ) override;

	/** Toggles whether or not this controller is being used to scrub sequencer */
	UE_API void ToggleSequencerScrubbingMode();

	/** Returns whether or not this controller is being used to scrub sequencer */
	UE_API bool IsScrubbingSequencer() const;

	/** Get the motioncontroller component of this interactor */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API class UMotionControllerComponent* GetMotionControllerComponent() const;


	/** Sets the EControllerHand for this motioncontroller */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API void SetControllerHandSide( const FName InControllerHandSide );

	/** Sets the EControllerHand for this motioncontroller */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API FName GetControllerHandSide() const;

	/** Returns the slide delta for pushing and pulling objects. Needs to be implemented by derived classes (e.g. touchpad for vive controller or scrollweel for mouse ) */
	UFUNCTION( CallInEditor, BlueprintNativeEvent, Category = "UVREditorInteractor" )
	UE_API float GetSlideDelta() const;

	/** Starts haptic feedback for physical motion controller */
	UE_API virtual void PlayHapticEffect( const float Strength ) override;

	/** Set if we want to force to show the laser*/
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API void SetForceShowLaser( const bool bInForceShow );

	// Tells us if thie interator is carrying a "carryable" actor, like a camera, which matches the interactor motions instead of using manipulators.
	UE_API bool IsCarrying() const;

	//
	// Trackpad
	//

	/**
	 * Gets the trackpad delta of the axis passed.
	 *
	 * @param Axis	The axis of which we want the slide delta. 0 is X axis and 1 is Y axis.Default is axis Y
	 */
	UE_API float GetTrackpadSlideDelta( const bool Axis = 1 ) const;

	/** Resets all the trackpad related values to default. */
	UE_API void ResetTrackpad();

	/** Check if the touchpad is currently touched */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API bool IsTouchingTrackpad() const;

	/** Get the current position of the trackpad or analog stick */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API FVector2D GetTrackpadPosition() const;

	/** Get the last position of the trackpad or analog stick */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API FVector2D GetLastTrackpadPosition() const;

	/** If the trackpad values are valid */
	UE_API bool IsTrackpadPositionValid( const int32 AxisIndex ) const;

	/** Get when the last time the trackpad position was updated */
	UE_API FTimespan& GetLastTrackpadPositionUpdateTime();

	/** Get when the last time the trackpad position was updated */
	UE_API FTimespan& GetLastActiveTrackpadUpdateTime();


	//
	// Getters and setters
	//
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API const FVector& GetLaserStart() const;

	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API const FVector& GetLaserEnd() const;

	/** Next frame this will be used as color for the laser */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API void SetForceLaserColor( const FLinearColor& InColor );

	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API AVREditorTeleporter* GetTeleportActor();

	/** Get the side of the controller */
	UFUNCTION( BlueprintCallable, Category = "UVREditorInteractor" )
	UE_API EControllerHand GetControllerSide() const;

	/** Returns what controller type this is for asymmetric control schemes */
	UFUNCTION( BlueprintCallable, Category = "VREditorInteractor" )
	UE_API EControllerType GetControllerType() const;

	/** Set what controller type this is for asymmetric control schemes */
	UFUNCTION( BlueprintCallable, Category = "VREditorInteractor" )
	UE_API void SetControllerType( const EControllerType InControllerType );

	/**
	 * Temporary set what controller type this is for asymmetric control schemes.
	 * You can't override the controller type when there's already an override.
	 * Remove the temporary controller type with EControllerType::Unknown
	 * @return true if the controller type was changed
	 */
	UFUNCTION( BlueprintCallable, Category = "VREditorInteractor" )
	UE_API bool TryOverrideControllerType( const EControllerType InControllerType );

	/** Gets if this interactor is hovering over UI */
	UFUNCTION( BlueprintCallable, Category = "VREditorInteractor" )
	UE_API bool IsHoveringOverUI() const;

	/** Sets if the quick menu is on this interactor */
	UE_API void SetHasUIOnForearm( const bool bInHasUIOnForearm );

	/** Check if the quick menu is on this interactor */
	UE_API bool HasUIOnForearm() const;

	/** Gets the current hovered widget component if any */
	UE_API UWidgetComponent* GetLastHoveredWidgetComponent() const;

	/** Sets the current hovered widget component */
	UE_API void SetLastHoveredWidgetComponent( UWidgetComponent* NewHoveringOverWidgetComponent );

	/** Sets if the interactor is clicking on any UI */
	UE_API void SetIsClickingOnUI( const bool bInIsClickingOnUI );

	/** Gets if the interactor is clicking on any UI */
	UFUNCTION( BlueprintCallable, Category = "VREditorInteractor" )
	UE_API bool IsClickingOnUI() const;

	/** Sets if the interactor is hovering over any UI */
	UE_API void SetIsHoveringOverUI( const bool bInIsHoveringOverUI );

	/** Sets if the interactor is right  hover over any UI */
	UE_API void SetIsRightClickingOnUI( const bool bInIsRightClickingOnUI );

	/** Gets if the interactor is right clicking on UI */
	UE_API bool IsRightClickingOnUI() const;

	/** Sets the time the interactor last pressed on UI */
	UE_API void SetLastUIPressTime( const double InLastUIPressTime );

	/** Gets last time the interactor pressed on UI */
	UE_API double GetLastUIPressTime() const;

	/** Sets the UI scroll velocity */
	UE_API void SetUIScrollVelocity( const float InUIScrollVelocity );

	/** Gets the UI scroll velocity */
	UE_API float GetUIScrollVelocity() const;

	/** Gets the trigger value */
	UFUNCTION( BlueprintCallable, Category = "VREditorInteractor" )
	UE_API float GetSelectAndMoveTriggerValue() const;

	/* ViewportInteractor overrides, checks if the laser is blocked by UI */
	UE_API virtual bool GetIsLaserBlocked() const override;

	/** Replace the default VR controller mesh with a custom one. */
	UFUNCTION(BlueprintCallable, Category = "VREditorInteractor")
	UE_API void ReplaceHandMeshComponent(UStaticMesh* NewMesh, FVector MeshScale = FVector(1.f, 1.f, 1.f));

	UE_API bool IsActionKeyPressed(FName ActionName) const;

protected:

	/** Polls input for the motion controllers transforms */
	UE_API virtual void PollInput() override;

	/** Whether swiping left/right on touchpad/joystick triggers undo/redo  */
	UPROPERTY(EditDefaultsOnly, Category = "VREditorInteractor")
	bool bIsUndoRedoSwipeEnabled;

	/** Motion controller component which handles late-frame transform updates of all parented sub-components */
	UPROPERTY()
	TObjectPtr<class UMotionControllerComponent> MotionControllerComponent;

	/** Separate motion controller component set to the "aim" pose motion source, used for the laser pointer. */
	UPROPERTY()
	TObjectPtr<class UMotionControllerComponent> LaserMotionControllerComponent;

	//
	// Graphics
	//

	/** Access to the current handmesh. Use ReplaceHandMeshComponent() to update the entire StaticMeshComponent. */
	UPROPERTY(BlueprintReadWrite, Category = "VREditorInteractor")
	TObjectPtr<class UStaticMeshComponent> HandMeshComponent;

	UFUNCTION(BlueprintNativeEvent)
	UE_API void UpdateHandMeshRelativeTransform();

	FVector3d HandMeshBaseScale;
	FTransform HandMeshGripTransform;


private:

	/** Spline for this hand's laser pointer */
	UPROPERTY()
	TObjectPtr<class USplineComponent> LaserSplineComponent;

	/** Spline meshes for curved laser */
	UPROPERTY()
	TArray<TObjectPtr<class USplineMeshComponent>> LaserSplineMeshComponents;

	/** MID for laser pointer material (opaque parts) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> LaserPointerMID;

	/** MID for laser pointer material (translucent parts) */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> TranslucentLaserPointerMID;

	/** Hover impact indicator mesh */
	UPROPERTY()
	TObjectPtr<class UStaticMeshComponent> HoverMeshComponent;

	/** Hover point light */
	UPROPERTY()
	TObjectPtr<class UPointLightComponent> HoverPointLightComponent;

	/** MID for hand mesh */
	UPROPERTY()
	TObjectPtr<class UMaterialInstanceDynamic> HandMeshMID;

	/** Avatar actor that we're attached to. Cached so that we can change the HandMeshComponent via BP at runtime. */
	UPROPERTY()
	TObjectPtr<AActor> OwningAvatar;

	/** True if this hand has a motion controller (or both!) */
	bool bHaveMotionController;

	// Special key action names for motion controllers
	static UE_API const FName TrackpadPositionX;
	static UE_API const FName TrackpadPositionY;
	static UE_API const FName TriggerAxis;
	static UE_API const FName MotionController_Left_FullyPressedTriggerAxis;
	static UE_API const FName MotionController_Right_FullyPressedTriggerAxis;
	static UE_API const FName MotionController_Left_PressedTriggerAxis;
	static UE_API const FName MotionController_Right_PressedTriggerAxis;

	/** Is the button for the specified action currently held down? */
	TMap<FName, bool> ActionKeysPressed;

	/** Is the Modifier button held down? */
	UE_DEPRECATED(5.1, "Use IsActionKeyPressed(VRActionTypes::Modifier) instead.")
	bool bIsModifierPressed;

	/** Current trigger pressed amount for 'select and move' (0.0 - 1.0) */
	float SelectAndMoveTriggerValue;

	FVector LaserStart;

	FVector LaserEnd;

private:

	/** Changes the color of the buttons on the handmesh */
	UE_API void ApplyButtonPressColors( const FViewportActionKeyInput& Action );

	/** Set the visuals for a button on the motion controller */
	UE_API void SetMotionControllerButtonPressedVisuals( const EInputEvent Event, const FName& ParameterName, const float PressStrength );

	/** Pops up some help text labels for the controller in the specified hand, or hides it, if requested */
	UE_API void ShowHelpForHand( const bool bShowIt );

	/** Called every frame to update the position of any floating help labels */
	UE_API void UpdateHelpLabels();

	/** Given a mesh and a key name, tries to find a socket on the mesh that matches a supported key */
	UE_API UStaticMeshSocket* FindMeshSocketForKey( UStaticMesh* StaticMesh, const FKey Key );

	/** Updates all the segments of the curved laser */
	UE_API void UpdateSplineLaser( const FVector& InStartLocation, const FVector& InEndLocation, const FVector& InForward );

	/** Sets the visibility on all curved laser segments */
	UE_API void SetLaserVisibility( const bool bVisible );

	/** Sets the visuals of the LaserPointer */
	UE_API void SetLaserVisuals( const FLinearColor& NewColor, const float CrawlFade, const float CrawlSpeed );

	/** Updates the radial menu */
	UE_API void UpdateRadialMenuInput( const float DeltaTime );

	/** Initialize shared HandMeshComponent settings */
	UE_API void SetHandMeshComponentProperties();

	/** Start undo or redo from swipe for the Vive */
	UE_API void UndoRedoFromSwipe( const ETouchSwipeDirection InSwipeDirection );

	//
	// General input @todo: VREditor: Should this be general (non-UI) in interactordata ?
	//

	/** For asymmetrical systems - what type of controller this is */
	UPROPERTY()
	EControllerType ControllerType;

	/** What was our previous controller type */
	UPROPERTY()
	EControllerType OverrideControllerType;

	//
	// UI
	//

	/** True if a floating UI panel is attached to the front of the hand, and we shouldn't bother drawing a laser
	pointer or enabling certain other features */
	bool bHasUIInFront;

	/** True if a floating UI panel is attached to our forearm, so we shouldn't bother drawing help labels */
	bool bHasUIOnForearm;

	/** True if we're currently holding the 'SelectAndMove' button down after clicking on UI */
	bool bIsClickingOnUI;

	/** When bIsClickingOnUI is true, this will be true if we're "right clicking".  That is, the Modifier key was held down at the time that the user clicked */
	bool bIsRightClickingOnUI;

	/** True if we're hovering over UI right now.  When hovering over UI, we don't bother drawing a see-thru laser pointer */
	bool bIsHoveringOverUI;

	/** Inertial scrolling -- how fast to scroll the mousewheel over UI */
	float UIScrollVelocity;

	/** Last real time that we pressed the 'SelectAndMove' button on UI.  This is used to detect double-clicks. */
	double LastUIPressTime;

protected:

	//
	// Trackpad support
	//

	/** True if the trackpad is actively being touched */
	UE_DEPRECATED(5.1, "Use IsActionKeyPressed(VRActionTypes::Touch) instead.")
	bool bIsTouchingTrackpad;

	/** True if pressing trackpad button (or analog stick button is down) */
	UE_DEPRECATED(5.1, "Use IsActionKeyPressed(VRActionTypes::ConfirmRadialSelection) instead.")
	bool bIsPressingTrackpad;

	/** Position of the touched trackpad */
	FVector2D TrackpadPosition;

	/** Last position of the touched trackpad */
	FVector2D LastTrackpadPosition;

	/** True if we have a valid trackpad position (for each axis) */
	bool bIsTrackpadPositionValid[2];

	/** Real time that the last trackpad position was last updated.  Used to filter out stale previous data. */
	FTimespan LastTrackpadPositionUpdateTime;

	/** Real time that the last trackpad position was over the dead zone threshold. */
	FTimespan LastActiveTrackpadUpdateTime;

	/** Forcing to show laser */
	bool bForceShowLaser;

	/** The color that will be used for one frame */
	TOptional<FLinearColor> ForceLaserColor;

	/**Whether a flick action was executed */
	bool bFlickActionExecuted;

	/** whether or not this controller is being used to scrub sequencer */
	bool bIsScrubbingSequence;

	//
	// Help
	//

	/** Right or left hand */
	UPROPERTY()
	FName ControllerMotionSource;

	/** True if we want help labels to be visible right now, otherwise false */
	bool bWantHelpLabels;

	/** Help labels for buttons on the motion controllers */
	TMap< FKey, class AFloatingText* > HelpLabels;

	/** Time that we either started showing or hiding help labels (for fade transitions) */
	FTimespan HelpLabelShowOrHideStartTime;

	//
	// Trigger axis state
	//

	/** True if trigger is fully pressed right now (or within some small threshold) */
	bool bIsTriggerFullyPressed;

	/** True if the trigger is currently pulled far enough that we consider it in a "half pressed" state */
	bool bIsTriggerPressed;

	/** True if trigger has been fully released since the last press */
	bool bHasTriggerBeenReleasedSinceLastPress;

	//
	// Swipe
	//

	/** Initial position when starting to touch the trackpad */
	FVector2D InitialTouchPosition;

	/** Latest swipe direction on the trackpad */
	ETouchSwipeDirection LastSwipe;

	/** The mode that owns this interactor */
	UPROPERTY()
	TObjectPtr<class UVREditorMode> VRMode;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API

