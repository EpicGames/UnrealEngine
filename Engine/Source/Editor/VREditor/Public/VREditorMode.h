// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "Editor/UnrealEdTypes.h"
#include "EditorWorldExtension.h"
#include "ShowFlags.h"
#include "Misc/App.h"
#include "Widgets/SWindow.h"
#include "HeadMountedDisplayTypes.h"
#include "UI/VRRadialMenuHandler.h"
#include "VREditorModeBase.h"
#include "VREditorMode.generated.h"

#define UE_API VREDITOR_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class AActor;
class FEditorViewportClient;
class SLevelViewport;
enum class EAutoChangeMode : uint8;
class UStaticMesh;
class UStaticMeshComponent;
class USoundBase;
class UMaterialInterface;

// Forward declare the GizmoHandleTypes that is defined in VIBaseTransformGizmo.h
enum class EGizmoHandleTypes : uint8;

/**
 * Types of actions that can be performed with VR controller devices
 */
namespace UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") VRActionTypes
{
	static const FName Touch( "Touch" );
	static const FName Modifier( "Modifier" );
	static const FName Modifier2( "Modifier2" );
	static const FName ConfirmRadialSelection( "ConfirmRadialSelection" );
	static const FName TrackpadPositionX( "TrackpadPosition_X" );
	static const FName TrackpadPositionY( "TrackpadPosition_Y" );
	static const FName TrackpadUp("TrackpadUp");
	static const FName TrackpadDown("TrackpadDown");
	static const FName TrackpadRight("TrackpadRight");
	static const FName TrackpadLeft("TrackpadLeft");
	static const FName TriggerAxis( "TriggerAxis" );
}

/**
 * VR Editor Mode. Extends editor viewports with functionality for VR controls and object manipulation
 */
UCLASS(MinimalAPI, Abstract, Transient)
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") UVREditorMode : public UVREditorModeBase
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UE_API UVREditorMode();

	/** Overrides the HMD device type, which is otherwise derived from the XR system name. */
	UE_API void SetHMDDeviceTypeOverride( FName InOverrideType );

	//~ Begin UEditorWorldExtension interface

	/** Initialize the VREditor */
	UE_API virtual void Init() override;

	/** Shutdown the VREditor */
	UE_API virtual void Shutdown() override;

protected:
	UE_API virtual void TransitionWorld(UWorld* NewWorld, EEditorWorldExtensionTransitionState TransitionState) override;

	//~ End UEditorWorldExtension interface

public:
	virtual bool NeedsSyntheticDpad()
	{
		return false;
	}

	UE_DEPRECATED(5.1, "This method is no longer used; the associated warning is no longer displayed.")
	virtual bool ShouldDisplayExperimentalWarningOnEntry() const { return true; }

	/** When the user actually enters the VR Editor mode */
	UE_API virtual void Enter() override;

	/** When the user leaves the VR Editor mode */
	UE_API virtual void Exit( bool bShouldDisableStereo ) override;

	/** Tick before the ViewportWorldInteraction is ticked */
	UE_API void PreTick( const float DeltaTime );

	/** Tick after the ViewportWorldInteraction is ticked */
	UE_API void PostTick( const float DeltaTime );

	/** Returns true if the user wants to exit this mode */
	virtual bool WantsToExitMode() const override
	{
		return bWantsToExitMode;
	}

	UE_API void AllocateInteractors();

	/** Call this to start exiting VR mode */
	UE_API void StartExitingVRMode();

	UE_API virtual bool GetLaserForHand(EControllerHand InHand, FVector& OutLaserStart, FVector& OutLaserEnd) const override;

	/** Gets the world space transform of the calibrated VR room origin.  When using a seated VR device, this will feel like the
	camera's world transform (before any HMD positional or rotation adjustments are applied.) */
	UE_API virtual FTransform GetRoomTransform() const override;

	/** Sets a new transform for the room, in world space.  This is basically setting the editor's camera transform for the viewport */
	UE_API void SetRoomTransform( const FTransform& NewRoomTransform );

	/** Gets the transform of the user's HMD in room space */
	UE_API FTransform GetRoomSpaceHeadTransform() const;

	/**
	 * Gets the world space transform of the HMD (head)
	 *
	 * @return	World space space HMD transform
	 */
	UE_API virtual FTransform GetHeadTransform() const override;

	/** Gets access to the world interaction system (const) */
	UE_API const class UViewportWorldInteraction& GetWorldInteraction() const;

	/** Gets access to the world interaction system */
	UE_API class UViewportWorldInteraction& GetWorldInteraction();

	/** If the mode was completely initialized */
	UE_API bool IsFullyInitialized() const;

	/** * Gets the tick handle to give external systems the change to be ticked right after the ViewportWorldInteraction is ticked */
	DECLARE_EVENT_OneParam(UVREditorMode, FOnVRTickHandle, const float /* DeltaTime */);
	FOnVRTickHandle& OnTickHandle()
	{
		return TickHandle;
	}

	/** Returns the Unreal controller ID for the motion controllers we're using */
	int32 GetMotionControllerID() const
	{
		return MotionControllerID;
	}

	/** Returns whether or not the flashlight is visible */
	bool IsFlashlightOn() const
	{
		return bIsFlashlightOn;
	}

	/** Returns the time since the VR Editor mode was last entered */
	inline FTimespan GetTimeSinceModeEntered() const
	{
		return FTimespan::FromSeconds( FApp::GetCurrentTime() ) - AppTimeModeEntered;
	}

	// @todo vreditor: Move this to FMath so we can use it everywhere
	// NOTE: OvershootAmount is usually between 0.5 and 2.0, but can go lower and higher for extreme overshots
	template<class T>
	static T OvershootEaseOut( T Alpha, const float OvershootAmount = 1.0f )
	{
		Alpha--;
		return 1.0f - ( ( Alpha * ( ( OvershootAmount + 1 ) * Alpha + OvershootAmount ) + 1 ) - 1.0f );
	}

	/** Gets access to the VR UI system (const) */
	const class UVREditorUISystem& GetUISystem() const
	{
		return *UISystem;
	}

	/** Gets access to the VR UI system */
	class UVREditorUISystem& GetUISystem()
	{
		return *UISystem;
	}

	/** Check whether the UISystem exists */
	bool UISystemIsActive() const
	{
		return UISystem != nullptr;
	}

	/** Lets other modules know if the radial menu is visible on a given interactor so input should be handled differently */
	UE_API bool IsShowingRadialMenu( const class UVREditorInteractor* Interactor ) const;

	/** Display the scene more closely to how it would appear at runtime (as opposed to edit time). */
	UFUNCTION(BlueprintCallable, Category="VREditorMode")
	UE_API void SetGameView(bool bGameView);

	/** Returns whether game view is currently active. */
	UFUNCTION(BlueprintCallable, Category="VREditorMode")
	UE_API bool IsInGameView() const;

	/** Gets the world scale factor, which can be multiplied by a scale vector to convert to room space */
	UFUNCTION( BlueprintCallable, Category = "VREditorMode" )
	UE_API float GetWorldScaleFactor() const;

	/** Spawns a flashlight on the specified hand */
	UE_API void ToggleFlashlight( class UVREditorInteractor* Interactor );

	/** Will update the TransformGizmo Actor with the next Gizmo type  */
	UE_API void CycleTransformGizmoHandleType();

	/** Gets the current Gizmo handle type */
	UE_API EGizmoHandleTypes GetCurrentGizmoType() const;

	/** @return Returns the type of HMD we're dealing with */
	UE_API FName GetHMDDeviceType() const;

	/** @return Checks to see if the specified interactor is aiming roughly toward the specified capsule */
	UE_API bool IsHandAimingTowardsCapsule(class UViewportInteractor* Interactor, const FTransform& CapsuleTransform, const FVector CapsuleStart, const FVector CapsuleEnd, const float CapsuleRadius, const float MinDistanceToCapsule, const FVector CapsuleFrontDirection, const float MinDotForAimingAtCapsule) const;

	/** Gets the hand interactor  */
	UE_API class UVREditorInteractor* GetHandInteractor( const EControllerHand ControllerHand ) const;

	/** Snaps the current selected actor to the ground */
	UE_API void SnapSelectedActorsToGround();

	/** Saved information about the editor and viewport we possessed, so we can restore it after exiting VR mode */
	struct FSavedEditorState : public FBaseSavedEditorState
	{
		float DragTriggerDistance = 0.0f;
		float TransformGizmoScale = 1.0f;
		bool bKeyAllEnabled = false;
		EAutoChangeMode AutoChangeMode;
	};

	virtual TSharedRef<FBaseSavedEditorState> CreateSavedState() override { return MakeShared<FSavedEditorState>(); }

	/** Gets the saved editor state from entering the mode */
	UE_API const FSavedEditorState& GetSavedEditorState() const;

	DECLARE_DELEGATE(FOnVREditingModeExit);
	/** Used to override dockable area restoration behavior */
	FOnVREditingModeExit OnVREditingModeExit_Handler;

	UE_API void SaveSequencerSettings(bool bInKeyAllEnabled, EAutoChangeMode InAutoChangeMode, const class USequencerSettings& InSequencerSettings);

	/** Create a static motion controller mesh for the current HMD platform */
	UE_API UStaticMeshComponent* CreateMotionControllerMesh( AActor* OwningActor, USceneComponent* AttachmentToComponent, UStaticMesh* OptionalControllerMesh = nullptr );

	/** Helper functions to create a static mesh */
	UE_API UStaticMeshComponent* CreateMesh( AActor* OwningActor, const FString& MeshName, USceneComponent* AttachmentToComponent /*= nullptr */ );
	UE_API UStaticMeshComponent* CreateMesh(AActor* OwningActor, UStaticMesh* Mesh, USceneComponent* AttachmentToComponent /*= nullptr */);

	/** Gets the container for all the assets of VREditor. */
	UE_API const class UVREditorAssetContainer& GetAssetContainer() const;

	/** Loads and returns the container for all the assets of VREditor. */
	static UE_API class UVREditorAssetContainer& LoadAssetContainer();

	/** Plays sound at location. */
	UE_API void PlaySound(USoundBase* SoundBase, const FVector& InWorldLocation, const float InVolume = 1.0f);

	/** Delegate to be called when a material is placed **/
	DECLARE_EVENT_ThreeParams( UVREditorPlacement, FOnPlaceDraggedMaterial, UPrimitiveComponent*, UMaterialInterface*, bool& );
	FOnPlaceDraggedMaterial& OnPlaceDraggedMaterial() { return OnPlaceDraggedMaterialEvent; };

	/** Delegate to be called when a preview actor is placed **/
	DECLARE_EVENT_OneParam(UVREditorPlacement, FOnPlacePreviewActor, bool);
	FOnPlacePreviewActor& OnPlacePreviewActor() { return OnPlacePreviewActorEvent; };

	/** Return true if currently aiming to teleport. */
	UE_API bool IsAimingTeleport() const;
	UE_API bool IsTeleporting() const;

	/** Toggles the debug mode. */
	static UE_API void ToggleDebugMode();

	/** Returns if the VR Mode is in debug mode. */
	static UE_API bool IsDebugModeEnabled();

	/** Get Teleporter */
	UE_API class AVREditorTeleporter* GetTeleportActor();

	/** Delegate to be called when the debug mode is toggled. */
	DECLARE_EVENT_OneParam(UVREditorMode, FOnToggleVRModeDebug, bool);
	FOnToggleVRModeDebug& OnToggleDebugMode() { return OnToggleDebugModeEvent; };

	const TArray<UVREditorInteractor*> GetVRInteractors() const
	{
		return Interactors;
	}

private:

	/** Called when the editor is closed */
	UE_API void OnEditorClosed();

	/** Called when someone closes a standalone VR Editor window */
	UE_API void OnVREditorWindowClosed( const TSharedRef<SWindow>& ClosedWindow );

	/** Restore the world to meters to the saved one when entering VR Editor */
	UE_API void RestoreWorldToMeters();

protected:

	//
	// Startup/Shutdown
	//

	UE_API void BeginEntry();
	UE_API void SetupSubsystems();
	UE_API void FinishEntry();

	/** The VR editor window, if it's open right now */
	TWeakPtr< class SWindow > VREditorWindowWeakPtr;

	/** True if we currently want to exit VR mode.  This is used to defer exiting until it is safe to do that */
	bool bWantsToExitMode;

	/** True if VR mode is fully initialized and ready to render */
	bool bIsFullyInitialized;

	/** App time that we entered this mode */
	FTimespan AppTimeModeEntered;

	//
	// Avatar visuals
	//

	/** Actor with components to represent the VR avatar in the world, including motion controller meshes */
	UPROPERTY()
	TObjectPtr<class AVREditorAvatarActor> AvatarActor;


	//
	// Flashlight
	//

	/** Spotlight for the flashlight */
	class USpotLightComponent* FlashlightComponent;

	/** If there is currently a flashlight in the scene */
	bool bIsFlashlightOn;

	//
	// Input
	//

	/** The Unreal controller ID for the motion controllers we're using */
	int32 MotionControllerID;


	//
	// Subsystems registered
	//

	FOnVRTickHandle TickHandle;

	/** Event broadcast when a material is placed */
	FOnPlaceDraggedMaterial OnPlaceDraggedMaterialEvent;

	/** Event broadcast when a preview actor is placed */
	FOnPlacePreviewActor OnPlacePreviewActorEvent;

	//
	// Subsystems
	//

	/** VR UI system */
	UPROPERTY()
	TObjectPtr<class UVREditorUISystem> UISystem;

	/** Teleporter system */
	UPROPERTY()
	TObjectPtr<class AVREditorTeleporter> TeleportActor;

	/** Automatic scale system */
	UPROPERTY()
	TObjectPtr<class UVREditorAutoScaler> AutoScalerSystem;

	//
	// World interaction
	//

	/** World interaction manager */
	UPROPERTY()
	TObjectPtr<class UViewportWorldInteraction> WorldInteraction;

	/** The current Gizmo type that is used for the TransformGizmo Actor */
	EGizmoHandleTypes CurrentGizmoType;

	UPROPERTY()
	TObjectPtr<class UVREditorPlacement> PlacementSystem;

	//
	// Interactors
	//

	UPROPERTY()
	TArray<TObjectPtr<UVREditorInteractor>> Interactors;

	//
	// Colors
	//
public:
	// Color types
	enum EColors
	{
		DefaultColor,
		SelectionColor,
		WorldDraggingColor,
		UIColor,
		UISelectionBarColor,
		UISelectionBarHoverColor,
		UICloseButtonColor,
		UICloseButtonHoverColor,
		TotalCount
	};

	// Gets the color
	UE_API FLinearColor GetColor( const EColors Color ) const;

	// Get the default near clipping plane for VR editing
	UE_API float GetDefaultVRNearClipPlane() const;

	/** Runtime and plugin modules can force VR Editor to refresh using this function */
	UE_API void RefreshVREditorSequencer(class ISequencer* InCurrentSequencer);

	/** Refresh the current actor preview widget on an in-world UI panel */
	UE_API void RefreshActorPreviewWidget(TSharedRef<SWidget> InWidget, int32 Index, AActor *Actor, bool bIsPanelDetached = false);

	/** General way to spawn an external UMG UI from a radial menu */
	UE_API void UpdateExternalUMGUI(const struct FVREditorFloatingUICreationContext& CreationContext);

	/** General way to spawn an external Slate UI from a radial menu */
	UE_API void UpdateExternalSlateUI(TSharedRef<SWidget> InWidget, FName Name, FVector2D InSize);

	/** Returns the currently active sequencer */
	UE_API class ISequencer* GetCurrentSequencer();

	/** The asset container path */
	static UE_API const TCHAR* AssetContainerPath;

	/** The controller to use when UnrealEd is in VR mode. Use VREditorInteractor get default editor behavior, or select a custom controller for special behavior */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, NoClear, Category="Classes" )
	TSoftClassPtr<UVREditorInteractor> InteractorClass;

	/** The teleporter to use when UnrealEd is in VR mode. Use VREditorTeleporter to get default editor behavior, or select a custom teleporter */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, NoClear, Category="Classes" )
	TSoftClassPtr<AVREditorTeleporter> TeleporterClass;

private:

	// All the colors for this mode
	TArray<FLinearColor> Colors;

	/** If this is the first tick or before */
	bool bFirstTick;

	/** Pointer to the current Sequencer */
	class ISequencer* CurrentSequencer;


	/** Container of assets */
	UPROPERTY()
	TObjectPtr<class UVREditorAssetContainer> AssetContainer;

	/** Whether currently in debug mode or not. */
	static UE_API bool bDebugModeEnabled;

	/** Event that gets broadcasted when debug mode is toggled. */
	FOnToggleVRModeDebug OnToggleDebugModeEvent;

	/** Overridden HMD device type. If NAME_None, HMD device type is derived from the XR system name. */
	FName HMDDeviceTypeOverride = NAME_None;

	bool bAddedViewportWorldInteractionExtension = false;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API

