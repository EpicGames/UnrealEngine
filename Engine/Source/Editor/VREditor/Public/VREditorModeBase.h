// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/UnrealEdTypes.h"
#include "EditorWorldExtension.h"
#include "HeadMountedDisplayTypes.h"
#include "ShowFlags.h"
#include "VREditorModeBase.generated.h"

#define UE_API VREDITOR_API


class SLevelViewport;


/**
 */
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, Transient)
class UVREditorModeBase : public UEditorWorldExtension
{
	GENERATED_BODY()

protected:
	struct FBaseSavedEditorState;

public:
	//~ Begin UEditorWorldExtension interface
	UE_API virtual void Init() override;
	UE_API virtual void Shutdown() override;
	//~ End UEditorWorldExtension interface

	UE_API virtual void Enter();
	UE_API virtual void Exit(bool bShouldDisableStereo);

	/** Sets whether we should actually use an HMD. Call this before activating VR mode */
	virtual void SetActuallyUsingVR(bool bShouldActuallyUseVR) { bActuallyUsingVR = bShouldActuallyUseVR; }

	/** Returns true if we're actually using VR, or false if we're faking it */
	virtual bool IsActuallyUsingVR() const { return bActuallyUsingVR; }

	/** Returns true if the user wants to exit this mode */
	virtual bool WantsToExitMode() const { return false; }

	/** Returns the transform which takes poses from XR tracking space ("room" space) to world space. */
	virtual FTransform GetRoomTransform() const { return FTransform::Identity; }

	/** Returns the user's head pose in world space. */
	virtual FTransform GetHeadTransform() const { return FTransform::Identity; }

	/** Gives start and end points in world space for an active laser, or returns false if there is no active laser for the specified hand. */
	virtual bool GetLaserForHand(EControllerHand InHand, FVector& OutLaserStart, FVector& OutLaserEnd) const { return false; }

	/** Delegate to be called when async VR mode entry has been completed. */
	DECLARE_MULTICAST_DELEGATE(FOnVRModeEntryComplete);
	FOnVRModeEntryComplete& OnVRModeEntryComplete() { return OnVRModeEntryCompleteEvent; }

	// TODO: Deprecate these in favor of replacements below; viewport may be null.
	UE_API const class SLevelViewport& GetLevelViewportPossessedForVR() const;
	UE_API class SLevelViewport& GetLevelViewportPossessedForVR();

	TSharedPtr<SLevelViewport> GetVrLevelViewport() { return VREditorLevelViewportWeakPtr.Pin(); }
	TSharedPtr<const SLevelViewport> GetVrLevelViewport() const { return VREditorLevelViewportWeakPtr.Pin(); }

protected:
	/** Derived classes can override this to return a derived state struct, and add their own saved state. */
	virtual TSharedRef<FBaseSavedEditorState> CreateSavedState() { return MakeShared<FBaseSavedEditorState>(); }

	/** Gets the saved editor state from entering the mode */
	FBaseSavedEditorState& SavedEditorStateChecked() { check(SavedEditorStatePtr); return *SavedEditorStatePtr; }
	const FBaseSavedEditorState& SavedEditorStateChecked() const { check(SavedEditorStatePtr); return *SavedEditorStatePtr; }

	/** Start using the viewport passed */
	UE_API virtual void StartViewport(TSharedPtr<SLevelViewport> Viewport);

	/** Close the current viewport */
	UE_API virtual void CloseViewport(bool bShouldDisableStereo);

	UE_API virtual void StereoViewportSetup(TSharedRef<SLevelViewport> Viewport);
	UE_API virtual void StereoViewportShutdown(TSharedRef<SLevelViewport> Viewport);

	UE_API virtual void EnableStereo();
	UE_API virtual void DisableStereo();

protected:
	/** True if we're in using an actual HMD in this mode, or false if we're "faking" VR mode for testing */
	bool bActuallyUsingVR;

	/** Delegate broadcast when async VR mode entry is completed. */
	FOnVRModeEntryComplete OnVRModeEntryCompleteEvent;

	TWeakPtr<SLevelViewport> VREditorLevelViewportWeakPtr;

	/** Saved information about the editor and viewport we possessed, so we can restore it after exiting VR mode */
	struct FBaseSavedEditorState
	{
		ELevelViewportType ViewportType = LVT_Perspective;
		FVector ViewLocation = FVector::ZeroVector;
		FRotator ViewRotation = FRotator::ZeroRotator;
		FEngineShowFlags ShowFlags = ESFIM_Editor;
		bool bLockedPitch = false;
		bool bAlwaysShowModeWidgetAfterSelectionChanges = false;
		float NearClipPlane = 0.0f;
		bool bRealTime = false;
		bool bOnScreenMessages = false;
		EHMDTrackingOrigin::Type TrackingOrigin = EHMDTrackingOrigin::Local;
		float WorldToMetersScale = 100.0f;
		bool bCinematicControlViewport = false;
	};

	TSharedPtr<FBaseSavedEditorState> SavedEditorStatePtr;
};

#undef UE_API
