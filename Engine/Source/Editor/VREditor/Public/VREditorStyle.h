// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "Styling/SlateStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ISlateStyle;
struct FSlateBrush;

/**  */
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") FVREditorStyle
{
public:

	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for the UMG Style */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();

	static FName GetSecondaryStyleSetName();

	static FName GetNumpadStyleSetName();

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return VREditorStyleInstance->GetBrush(PropertyName, Specifier);
	}

private:

	static TSharedRef< class FSlateStyleSet > Create();

private:

	static TSharedPtr< class FSlateStyleSet > VREditorStyleInstance;

	
};