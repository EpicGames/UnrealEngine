// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VISettings.h"
#include "VRModeSettings.generated.h"

#define UE_API VREDITOR_API


class AVREditorTeleporter;
class UVREditorInteractor;
class UVREditorModeBase;


UENUM()
enum class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") EInteractorHand : uint8
{
	/** Right hand */
	Right,

	/** Left hand */
	Left,
};

/**
* Implements the settings for VR Mode.
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(MinimalAPI, config = EditorSettings)
class UVRModeSettings : public UVISettings
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	GENERATED_BODY()

public:

	/** Default constructor that sets up CDO properties */
	UE_API UVRModeSettings();

	/**If true, wearing a Vive or Oculus Rift headset will automatically enter VR Editing mode */
	UPROPERTY(EditAnywhere, config, Category = "General", meta = (DisplayName = "Enable VR Mode Auto-Entry"))
	uint32 bEnableAutoVREditMode : 1;

	// Whether or not sequences should be autokeyed
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	uint32 bAutokeySequences : 1;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Which hand should have the primary interactor laser on it
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	EInteractorHand InteractorHand;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Show the movement grid for orientation while moving through the world */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	uint32 bShowWorldMovementGrid : 1;

	/** Dim the surroundings while moving through the world */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	uint32 bShowWorldMovementPostProcess : 1;

	/** Display a progress bar while scaling that shows your current scale */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	uint32 bShowWorldScaleProgressBar : 1;

	/** Adjusts the brightness of the UI panels */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	float UIBrightness;

	/** The size of the transform gizmo */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	float GizmoScale;

	/** The maximum time in seconds between two clicks for a double-click to register */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	float DoubleClickTime;

	/** The amount (between 0-1) you have to depress the Vive controller trigger to register a press */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	float TriggerPressedThreshold_Vive;

	/** The amount (between 0-1) you have to depress the Oculus Touch controller trigger to register a press */
	UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.")
	UPROPERTY(config)
	float TriggerPressedThreshold_Rift;

	/** The mode extension to use when UnrealEd is in VR mode. Use VREditorMode to get default editor behavior or select a custom mode. */
	UPROPERTY(EditAnywhere, config, NoClear, Category = "General")
	TSoftClassPtr<UVREditorModeBase> ModeClass;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

#undef UE_API
