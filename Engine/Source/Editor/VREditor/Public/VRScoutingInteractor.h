// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "HeadMountedDisplayTypes.h"
#include "VREditorInteractor.h"
#include "VRScoutingInteractor.generated.h"

#define UE_API VREDITOR_API

PRAGMA_DISABLE_DEPRECATION_WARNINGS


class AActor;
class UStaticMesh;
class UStaticMeshSocket;
class UInputComponent;

/**
 * Represents the interactor in the world
 */
UCLASS(MinimalAPI, Abstract)
class UE_DEPRECATED(5.7, "VR Editor mode is deprecated; use the XR Creative Framework plugin instead.") UVRScoutingInteractor : public UVREditorInteractor
{
	GENERATED_BODY()

public:

	/** Default constructor */
	UE_API UVRScoutingInteractor();

	/** Gets the trackpad slide delta */
	UE_API virtual float GetSlideDelta_Implementation() const override;

	/** Sets up all components */
	UE_API virtual void SetupComponent_Implementation(AActor* OwningActor) override;

	// IViewportInteractorInterface overrides
	UE_API virtual void Shutdown_Implementation() override;

	/** Sets the gizmo mode for selected object */
	UFUNCTION(BlueprintCallable, Category = "Scouting")
	UE_API void SetGizmoMode(EGizmoHandleTypes InGizmoMode);

	/** Gets the gizmo mode for selected object */
	UFUNCTION(BlueprintCallable, Category = "Scouting")
	UE_API EGizmoHandleTypes GetGizmoMode() const;

	/** Gets all actors that are selected in the world editor */
	UFUNCTION(BlueprintCallable, Category = "Scouting")
	static UE_API TArray<AActor*> GetSelectedActors();

	/** Shown in Navigation mode */
	UPROPERTY(Category = Interactor, EditAnywhere, BlueprintReadOnly)
	TObjectPtr<class UStaticMeshComponent> FlyingIndicatorComponent;

	/** Returns the current InputComponent. This will be NULL unless bReceivesEditorInput is set to true. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Input|Editor")
	UInputComponent* GetInputComponent() const
	{
		return EditorOnlyInputComponent.Get();
	}

	UFUNCTION(BlueprintSetter, Category = "Input|Editor")
	UE_API void SetReceivesEditorInput(bool bInValue);

	UFUNCTION(BlueprintGetter, BlueprintPure, Category = "Input|Editor")
	bool GetReceivesEditorInput() const
	{
		return bReceivesEditorInput;
	}

private:

	/** Creates the EditorOnlyInputComponent if it does not already exist and registers all subobject callbacks to it */
	UE_API void CreateEditorInput();

	/** Removes the EditorOnlyInputComponent from this object */
	UE_API void RemoveEditorInput();

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UInputComponent> EditorOnlyInputComponent;

	/** If set to true, then this interactor will be able to recieve input delegate callbacks when in the editor. Defaults to true since we will always want this interactor to consume input */
	UPROPERTY(EditAnywhere, Category = "Input|Editor", BlueprintSetter = SetReceivesEditorInput, BlueprintGetter = GetReceivesEditorInput)
	bool bReceivesEditorInput = true;

};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API

