// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class VREditor : ModuleRules
	{
        public VREditor(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "AppFramework",
				    "Core",
				    "CoreUObject",
					"ApplicationCore",
				    "Engine",
                    "InputCore",
				    "Slate",
					"SlateCore",
                    "RenderCore",
					"EditorFramework",
					"UnrealEd",
					"UMG",
					"LevelEditor",
					"HeadMountedDisplay",
					"Analytics",
                    "LevelSequence",
                    "Sequencer",
                    "Projects",
					"ToolMenus",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"ViewportInteraction",
					"HeadMountedDisplay"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AssetTools",
					"PlacementMode",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
                    "PlacementMode"
                }
			);
		}
	}
}
